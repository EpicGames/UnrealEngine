
; Do not put device profile properties in here. This file is only used for parts of the cooker
; that calls UDeviceProfileManager::Get().FindProfile with the TargetPlatform IniPlatformName and
; these settings redirect to the Windows version

[DeviceProfiles]
+DeviceProfileNameAndTypes=WinGDK,WinGDK
+DeviceProfileNameAndTypes=WinGDKClient,WinGDK

[WinGDK DeviceProfile]
DeviceType=WinGDK
BaseProfileName=Windows

[WinGDKClient DeviceProfile]
DeviceType=WinGDK
BaseProfileName=WindowsClient
