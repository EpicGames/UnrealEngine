[/Script/WindowsTargetPlatform.WindowsTargetSettings]
+AllowedMissingSymbolFiles="EOSSDK-WinGDK-Shipping.DLL"
+AllowedMissingSymbolFiles="EOSSDK-Win64-Shipping.DLL"
ManifestVersion=Version1

[StreamingInstall]
DefaultProviderName=GDKPackageChunkInstall

[PlatformFeatures]
PlatformFeaturesModule="MSGameStorePlatformFeatures"
SaveGameSystemModule="GDKSaveGameSystem"

[OnlineSubsystem]
DefaultPlatformService=GDK
NativePlatformService=GDK

[Internationalization]
+LocalizationPaths=../../../Engine/Platforms/WinGDK/Content/Localization/WinGDK

[PlatformDLC]
DefaultModule=GDKPlatformDLC

