[DataDrivenPlatformInfo]
bIsConfidential=true
IniParent=Windows
AdditionalRestrictedFolders=GDK
TargetSettingsIniSectionName=/Script/WindowsTargetPlatform.WindowsTargetSettings
Freezing_MaxFieldAlignment64=8
Freezing_bAlignBases=true
bHasDedicatedGamepad=false
bDefaultInputStandardKeyboard=true
bInputSupportConfigurable=true
DefaultInputType=MouseAndKeyboard
bSupportsMouseAndKeyboard=true
bSupportsGamepad=true
bCanChangeGamepadType=true
bSupportsTouch=false
GlobalIdentifier=5636FBC1D2B54F62AC8E7D4FB184B45A

NormalIconPath=/Platforms/WinGDK/Content/Editor/Slate/Launcher/Platform_WinGDK_24x
LargeIconPath=/Platforms/WinGDK/Content/Editor/Slate/Launcher/Platform_WinGDK_128x
XLargeIconPath=
AutoSDKPath=GDK
TutorialPath=SharingAndReleasing
Windows:bIsEnabled=true
Mac:bIsEnabled=false
Linux:bIsEnabled=false
bUsesHostCompiler=false
bUATClosesAfterLaunch=false
PlatformGroupName=Desktop
ICUDataVersion=icudt64l
