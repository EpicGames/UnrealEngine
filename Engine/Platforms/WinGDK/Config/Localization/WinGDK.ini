[CommonSettings]
SourcePath=Platforms/WinGDK/Content/Localization/WinGDK
DestinationPath=Platforms/WinGDK/Content/Localization/WinGDK
ManifestName=WinGDK.manifest
ArchiveName=WinGDK.archive
PortableObjectName=WinGDK.po
ResourceName=WinGDK.locres
NativeCulture=en
CulturesToGenerate=en
CulturesToGenerate=ja
CulturesToGenerate=ko
CulturesToGenerate=zh-Hans

;Gather text from source code
[GatherTextStep0]
CommandletClass=GatherTextFromSource
SearchDirectoryPaths=Platforms/WinGDK/Source/Runtime/
SearchDirectoryPaths=Platforms/WinGDK/Source/Developer/
SearchDirectoryPaths=Platforms/WinGDK/Plugins/
SearchDirectoryPaths=Platforms/WinGDK/Config/
SearchDirectoryPaths=Platforms/GDK/Source/Runtime/
SearchDirectoryPaths=Platforms/GDK/Source/Developer/
SearchDirectoryPaths=Platforms/GDK/Plugins/
ExcludePathFilters=Platforms/WinGDK/Config/Localization/*
ExcludePathFilters=Platforms/WinGDK/Plugins/Online/*
ExcludePathFilters=Platforms/GDK/Plugins/Online/*
ExcludePathFilters=*/NoRedist/*
ExcludePathFilters=*/NotForLicensees/*
ExcludePathFilters=*/LimitedAccess/*
ExcludePathFilters=*/ThirdParty/*
FileNameFilters=*.cpp
FileNameFilters=*.h
FileNameFilters=*.c
FileNameFilters=*.inl
FileNameFilters=*.mm
FileNameFilters=*.ini
ShouldGatherFromEditorOnlyData=true

;Write Manifest
[GatherTextStep1]
CommandletClass=GenerateGatherManifest

;Write Archives
[GatherTextStep2]
CommandletClass=GenerateGatherArchive
bPurgeOldEmptyEntries=true

;Import localized PO files
[GatherTextStep3]
CommandletClass=InternationalizationExport
bImportLoc=true

;Write Localized Text Resource
[GatherTextStep4]
CommandletClass=GenerateTextLocalizationResource

;Export PO files
[GatherTextStep5]
CommandletClass=InternationalizationExport
bExportLoc=true
