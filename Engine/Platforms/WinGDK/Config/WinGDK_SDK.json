{
	"ParentSDKFile": "../../GDK/Config/GDK_SDK.json",

	"//": "WinGDK goes further back than GDK",
	"MinSoftwareVersion": "10.0.18362.0"
}
