[Options]
+SyncCategory=(UniqueId="0526f65a-191f-4175-ab02-e309f967a2b6", Name="Platform Support: WinGDK", Paths=".../Platforms/WinGDK/...", Requires="a2000523-bbc7-4d2c-88a4-50da4e777ec6")
