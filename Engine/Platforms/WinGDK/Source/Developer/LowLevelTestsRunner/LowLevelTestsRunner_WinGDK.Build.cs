// Copyright Epic Games, Inc. All Rights Reserved.

namespace UnrealBuildTool.Rules
{
	public class LowLevelTestsRunner_WinGDK : LowLevelTestsRunner
	{
		public LowLevelTestsRunner_WinGDK(ReadOnlyTargetRules Target) : base(Target)
		{
			if (Target.Platform == UnrealTargetPlatform.WinGDK)
			{
				PrivateDependencyModuleNames.Add("GRDK");
			}
		}
	}
}
