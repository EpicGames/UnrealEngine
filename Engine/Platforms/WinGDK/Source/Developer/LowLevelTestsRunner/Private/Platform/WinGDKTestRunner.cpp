// Copyright Epic Games, Inc. All Rights Reserved.

#include "TestRunner.h"
#include "Misc/Paths.h"
#include "Windows/AllowWindowsPlatformTypes.h"
#include <XGameRuntimeInit.h>
#include <XError.h>
#include "Windows/HideWindowsPlatformTypes.h"

struct FGameRuntimeAutoInit
{
	FGameRuntimeAutoInit()
	{
		// change to the same directory as the executable
		TCHAR ExeFilePath[MAX_PATH];
		::GetModuleFileNameW(nullptr, ExeFilePath, MAX_PATH);
		::SetCurrentDirectoryW(*FPaths::GetPath(ExeFilePath));

		// GDKRuntime examine the commandline, so make sure it is initialized
		FCommandLine::Set(TEXT(""));

		// Don't need to shutdown module in this function, because all modules will be shutdown by module manager in RunTests
		FModuleManager::LoadModulePtr<IModuleInterface>(TEXT("GDKRuntime"));
	}

	~FGameRuntimeAutoInit()
	{
	}

} GGameRuntimeAutoInit;

// NB. actual launch is handled by WindowsTestRunner.cpp
