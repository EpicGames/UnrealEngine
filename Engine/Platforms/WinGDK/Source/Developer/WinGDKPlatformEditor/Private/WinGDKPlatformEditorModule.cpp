// Copyright Epic Games, Inc. All Rights Reserved.

#include "CoreMinimal.h"
#include "Modules/ModuleManager.h"
#include "Modules/ModuleInterface.h"
#include "ISettingsModule.h"
#include "Modules/ModuleManager.h"
#include "UObject/Object.h"
#include "WinGDKTargetSettings.h"
#include "UObject/WeakObjectPtr.h"
#include "UObject/Class.h"
#include "PropertyEditorModule.h"
#include "WinGDKTargetSettingsDetails.h"


#define LOCTEXT_NAMESPACE "FWinGDKPlatformEditorModule"

/**
* Module for WinGDK platform editor utilities
*/
class FWinGDKPlatformEditorModule
	: public IModuleInterface
{
	// IModuleInterface interface

	virtual void StartupModule() override
	{
		// register detail customization
		static FName PropertyEditor("PropertyEditor");
		FPropertyEditorModule* PropertyModule = FModuleManager::GetModulePtr<FPropertyEditorModule>(PropertyEditor);

		if (PropertyModule != nullptr)
		{
			static FName WinGDKTargetSettings("WinGDKTargetSettings");
			PropertyModule->RegisterCustomClassLayout(WinGDKTargetSettings, FOnGetDetailCustomizationInstance::CreateStatic(&FWinGDKTargetSettingsDetails::MakeInstance));
		}

		// register settings
		ISettingsModule* SettingsModule = FModuleManager::GetModulePtr<ISettingsModule>("Settings");

		if (SettingsModule != nullptr)
		{
			SettingsModule->RegisterSettings("Project", "Platforms", "Windows (GDK)",
				LOCTEXT("TargetSettingsName", "Windows (GDK)"),
				LOCTEXT("TargetSettingsDescription", "Windows (GDK) project settings"),
				GetMutableDefault<UWinGDKTargetSettings>()
			);
		}

	}

	virtual void ShutdownModule() override
	{
		ISettingsModule* SettingsModule = FModuleManager::GetModulePtr<ISettingsModule>("Settings");

		if (SettingsModule != nullptr)
		{
			SettingsModule->UnregisterSettings("Project", "Platforms", "Windows (GDK)");
		}
	}
};


IMPLEMENT_MODULE(FWinGDKPlatformEditorModule, WinGDKPlatformEditor);

#undef LOCTEXT_NAMESPACE
