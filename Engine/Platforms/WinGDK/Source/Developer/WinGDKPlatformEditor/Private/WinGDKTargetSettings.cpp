// Copyright Epic Games, Inc. All Rights Reserved.

#include "WinGDKTargetSettings.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(WinGDKTargetSettings)

UWinGDKTargetSettings::UWinGDKTargetSettings(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

