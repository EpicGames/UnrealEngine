// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "GDKTargetSettings.h"
#include "WinGDKTargetSettings.generated.h"

/**
 * Implements the settings for the Windows on GDK target platform.
 */
UCLASS(MinimalAPI, config = Engine, defaultconfig, ConfigDoNotCheckDefaults)
class UWinGDKTargetSettings
	: public UGDKTargetSettings
{
public:

	GENERATED_UCLASS_BODY()

	// Use WinGDK .ini files in the editor
	virtual const TCHAR* GetConfigOverridePlatform() const override
	{
		return TEXT("WinGDK");
	}

	virtual const TCHAR* GetPlatformName() const override
	{
		return TEXT("WinGDK");
	}

	void virtual OverrideConfigSection(FString& InOutSectionName) override
	{
		// WinGDK reads & writes the main Windows target settings to ensure platform parity as much as possible
		InOutSectionName = TEXT("/Script/WindowsTargetPlatform.WindowsTargetSettings");
	}


	/* Optional Content GUID. See the GDK documentation for details */
	UPROPERTY(EditAnywhere, Config, Category=PartnerCenter, AdvancedDisplay)
	FString ContentId;

	/* Whether submission validator expiration errors should occur in non-shipping builds */
	UPROPERTY(EditAnywhere, Config, Category=Packaging, AdvancedDisplay)
	bool bSubmissionValidatorExpirationError;

	/* This should not be used unless otherwise noted by your Account Manager.. */
	UPROPERTY(EditAnywhere, Config, Category=Packaging, AdvancedDisplay, Meta=(DisplayName="MSA Full Trust"))
	bool bMSAFullTrust;

	/* See the documentation for details. You must be authorized by your Account Manager to use this, or packages will not be accepted by Partner Center */
	UPROPERTY(EditAnywhere, Config, Category=Features, AdvancedDisplay)
	bool bUseModFolder;

	/* See the documentation for details. Will be displayed under %ProgramFiles%/ModifiableWindowsApps/<ModFolder> */
	UPROPERTY(EditAnywhere, Config, Category=Features, AdvancedDisplay, Meta=(EditCondition="bUseModFolder"))
	FString ModFolder;

	/**
	* Specifies if the title requires Xbox Live connectivity in order to run - most titles should not tick this.
	* If set, the application will fail to launch if the console cannot reach Xbox Live, and will terminate if it loses the connection at any time. 
	* This does not control if your title uses Xbox LIVE. 
	*/
	UPROPERTY(EditAnywhere, Config, Category=Features, AdvancedDisplay, meta=(DisplayName="Require permanent connectivity to Xbox Live"))
	bool bRequireXboxLive;




	/** Whether exceptions should be ultimately passed on to Windows Error Reporting, allowing crash callstacks to be uploaded to Partner Center */
	UPROPERTY(EditAnywhere, Config, Category=Features, AdvancedDisplay, Meta=(DisplayName="Use Windows Error Reporting"))
	bool bUseWER = true;


	/* Whether to always include DX11 dependency, regardless of the 'default rhi' setting. Use this if you plan to expose bUseD3D12InGame to users, or provide an alternative method of choosing an RHI */
	UPROPERTY(EditAnywhere, Config, Category = Packaging, Meta=(DisplayName="Always Include DX11 Dependency"))
	bool bAlwaysIncludeDX11Dependency = false;

	/* Whether to automatically include GameInputRedist in the custom install actions */
	UPROPERTY(EditAnywhere, Config, Category = Packaging, Meta=(DisplayName="Always Include GameInput Dependency"))
	bool bAlwaysIncludeGameInputDependency = false;

	/* Custom installation actions. See the GDK documentation for details */
	UPROPERTY(EditAnywhere, Config, Category=Packaging, Meta=(TitleProperty=Name))
	TArray<FGDKCustomInstallAction> CustomInstallActions;

	/* Relative path in the package where the custom install actions are staged. Defaults to a CustomInstall folder under the root of the package */
	UPROPERTY(EditAnywhere, Config, Category=Packaging)
	FString CustomInstallStagingRoot;

	/* Blocks the game from installing or launching on a Windows version older than this. Set to a four-part version number (e.g., 10.0.18362.0) or leave blank to use the GDK's default value. */ 
	UPROPERTY(EditAnywhere, Config, Category=Packaging, AdvancedDisplay)
	FString RequiredMinimumWindowsVersion;

	/* The Windows version displayed as the Minimum Requirements on Microsoft store fronts. Set to a four-part version number (e.g., 10.0.18362.0) or leave blank to use the GDK's default value. */
	UPROPERTY(EditAnywhere, Config, Category=Packaging, AdvancedDisplay)
	FString SuggestedMinimumWindowsVersion;

	/* The Windows version displayed as the Recommended Requirements on Microsoft store fronts. Set to a four-part version number (e.g., 10.0.18362.0) or leave blank to use the GDK's default value. */
	UPROPERTY(EditAnywhere, Config, Category=Packaging, AdvancedDisplay)
	FString RecommendedWindowsVersion;

	/**
	* Specifies if the shipping build configuration should skip suffixing the AppID with the configuration name.
	* It is recommended to leave this disabled unless there is a specific reason you need to skip suffixing on the shipping build.
	*/
	UPROPERTY(EditAnywhere, Config, Category=Packaging, AdvancedDisplay)
	bool bSkipAppIdSuffixForShippingConfig;

};
