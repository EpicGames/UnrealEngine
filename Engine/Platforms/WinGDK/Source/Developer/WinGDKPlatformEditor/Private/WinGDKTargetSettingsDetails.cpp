// Copyright Epic Games, Inc. All Rights Reserved.

#include "WinGDKTargetSettingsDetails.h"

#include "DetailWidgetRow.h"
#include "DetailCategoryBuilder.h"
#include "DetailLayoutBuilder.h"
#include "IDetailChildrenBuilder.h"
#include "IDetailGroup.h"
#include "ISettingsModule.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/Input/SHyperlink.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/SBoxPanel.h"
#include "WinGDKTargetSettings.h"
#include "PlatformInfo.h"
#include "Styling/AppStyle.h"

#define LOCTEXT_NAMESPACE "WinGDKTargetSettingsDetails"

FWinGDKTargetSettingsDetails::FWinGDKTargetSettingsDetails()
	: FGDKTargetSettingsDetails(TEXT("WinGDK"))
{
}


void FWinGDKTargetSettingsDetails::CustomizeDetails(IDetailLayoutBuilder& DetailBuilder)
{
	FGDKTargetSettingsDetails::CustomizeDetails(DetailBuilder);

	// lookup Windows platform info (just used for icon lookup)
	const PlatformInfo::FTargetPlatformInfo* WindowsPlatformInfo = PlatformInfo::FindPlatformInfo(TEXT("Windows"));

	// add a link to the Windows settings
	IDetailCategoryBuilder& WindowsSettingsShortcut = DetailBuilder.EditCategory("WindowsSettings", FText::GetEmpty(), ECategoryPriority::Important);
	WindowsSettingsShortcut.AddCustomRow(FText::GetEmpty(), false)
	.WholeRowWidget
	[
		SNew(SBorder)
		.BorderImage(FAppStyle::GetBrush("ToolPanel.GroupBorder"))
		[
			SNew(SHorizontalBox)

			// Windows platform icon
			+ SHorizontalBox::Slot()
			.AutoWidth()
			.VAlign(VAlign_Center)
			.Padding(8.4f)
			[
				SNew(SImage)
				.Image(WindowsPlatformInfo ? FAppStyle::GetBrush(WindowsPlatformInfo->GetIconStyleName(EPlatformIconSize::Normal)) : FStyleDefaults::GetNoBrush() )
			]

			// Link to Windows platform settings
			+SHorizontalBox::Slot()
			.AutoWidth()
			.VAlign(VAlign_Center)
			.Padding(8.4f)
			[
				SNew(SHyperlink)
				.Text(LOCTEXT("GoToWinSettings", "Additional settings are shared with the Windows platform"))
				.Style(FAppStyle::Get(), TEXT("NavigationHyperlink"))
				.ToolTipText(LOCTEXT("GoToWinSettingsTip", "Click here to open the Windows platform settings"))
				.OnNavigate_Lambda([]() 
				{ 
					ISettingsModule* SettingsModule = FModuleManager::GetModulePtr<ISettingsModule>("Settings");
					if (SettingsModule)
					{
						SettingsModule->ShowViewer(FName("Project"), FName("Platforms"), FName("Windows"));
					}
				})
			]
		]
	];
}

#undef LOCTEXT_NAMESPACE
