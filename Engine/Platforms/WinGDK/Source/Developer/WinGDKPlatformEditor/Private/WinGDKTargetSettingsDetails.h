// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "GDKTargetSettingsDetails.h"

/**
* Detail customization for Windows on GDK target settings panel
*/
class FWinGDKTargetSettingsDetails : public FGDKTargetSettingsDetails
{
public:
	FWinGDKTargetSettingsDetails();
	static TSharedRef<IDetailCustomization> MakeInstance()
	{
		return MakeShareable(new FWinGDKTargetSettingsDetails);
	}

	/** IDetailCustomization interface */
	virtual void CustomizeDetails(IDetailLayoutBuilder& DetailBuilder) override;

private:
};
