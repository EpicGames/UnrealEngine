// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class WinGDKPlatformEditor : ModuleRules
{
	public WinGDKPlatformEditor(ReadOnlyTargetRules Target) : base(Target)
	{
		BinariesSubFolder = "WinGDK";

		PrivateDependencyModuleNames.AddRange(
			new string[] {
				"Core",
				"CoreUObject",
				"DesktopPlatform",
				"Slate",
				"SlateCore",
				"PropertyEditor",
                "GDKPlatformEditor",
				"Settings",
				"DesktopPlatform",
            }
        );
	}
}
