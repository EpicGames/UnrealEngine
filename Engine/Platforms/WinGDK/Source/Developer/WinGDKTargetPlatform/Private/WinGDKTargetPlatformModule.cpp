// Copyright Epic Games, Inc. All Rights Reserved.

#include "CoreMinimal.h"
#include "Modules/ModuleManager.h"
#include "Interfaces/ITargetPlatformModule.h"
#include "Misc/Paths.h"
#include "Common/TargetPlatformBase.h"

#define LOCTEXT_NAMESPACE "FWinGDKTargetPlatformModule"


/**
 * Module for the WinGDK target platform.
 */
class FWinGDKTargetPlatformModule : public ITargetPlatformModule
{
public:

	virtual void GetTargetPlatforms(TArray<ITargetPlatform*>& TargetPlatforms) override
	{
	}

	virtual void GetTargetPlatforms(TArray<ITargetPlatform*>& TargetPlatforms, TArray<ITargetPlatformSettings*> TargetPlatformSettings, TArray<ITargetPlatformControls*> TargetPlatformControls)
	{
		for (ITargetPlatformControls* TargetPlatformControlsIt : TargetPlatformControls)
		{
			TargetPlatforms.Add(new FTargetPlatformMerged(TargetPlatformControlsIt->GetTargetPlatformSettings(), TargetPlatformControlsIt));
		}
	}

};


#undef LOCTEXT_NAMESPACE


IMPLEMENT_MODULE( FWinGDKTargetPlatformModule, WinGDKTargetPlatform);
