// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class WinGDKTargetPlatform : ModuleRules
{
	public WinGDKTargetPlatform(ReadOnlyTargetRules Target) : base(Target)
	{
		BinariesSubFolder = "WinGDK";
		SDKVersionRelevantPlatforms.Add(UnrealTargetPlatform.WinGDK);

		PrivateDependencyModuleNames.AddRange(
			new string[] {
				"Core",
				"DesktopPlatform",
				"TargetPlatform",
				"WinGDKTargetPlatformSettings",
				"WinGDKTargetPlatformControls",
			}
		);
	}
}
