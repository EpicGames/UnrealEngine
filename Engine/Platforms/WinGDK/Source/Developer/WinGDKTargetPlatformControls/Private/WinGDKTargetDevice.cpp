// Copyright Epic Games, Inc. All Rights Reserved.

#include "WinGDKTargetDevice.h"

DEFINE_LOG_CATEGORY_STATIC(LogWinGDKTargetDevice, Log, All);

FWinGDKTargetDevice::FWinGDKTargetDevice( const ITargetPlatformControls& InTargetPlatformControls)
	: Super(InTargetPlatformControls)
{
}

FWinGDKRemoteTargetDevice::FWinGDKRemoteTargetDevice( const ITargetPlatformControls& InTargetPlatformControls, const FString& DisplayName )
	: Super(InTargetPlatformControls, DisplayName)
{
}
