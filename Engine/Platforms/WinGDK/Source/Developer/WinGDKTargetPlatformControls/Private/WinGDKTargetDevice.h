// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "LocalPcTargetDevice.h"
#include "RemoteWindowsDevice.h"

/**
 * Implements a WinGDK target device.
 */
class FWinGDKTargetDevice : public FLocalPcTargetDevice
{
	typedef FLocalPcTargetDevice Super;

public:

	FWinGDKTargetDevice( const ITargetPlatformControls& InTargetPlatformControls );

	virtual ~FWinGDKTargetDevice() = default;

public:

	virtual FTargetDeviceId GetId() const override
	{
		return FTargetDeviceId(TargetPlatformControls.PlatformName(), Super::GetName());
	}

	virtual FString GetName() const override
	{
		return Super::GetName() + TEXT(" (WinGDK)");
	}
};

class FWinGDKRemoteTargetDevice : public FRemoteWindowsDevice
{
	typedef FRemoteWindowsDevice Super;

public:
	FWinGDKRemoteTargetDevice( const ITargetPlatformControls& InTargetPlatformControls, const FString& DisplayName );

	virtual FString GetName() const override
	{
		return Super::GetName() + TEXT(" (WinGDK)");
	}
};
