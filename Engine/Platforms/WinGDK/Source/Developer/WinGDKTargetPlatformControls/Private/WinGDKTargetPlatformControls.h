// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "GenericWindowsTargetPlatformControls.h"
#include "WinGDKPlatformProperties.h"
#include "WinGDKTargetDevice.h"

/**
 * Implements the WinGDK target platform controls.
 */
template<bool IS_CLIENT_ONLY>
class FWinGDKTargetPlatformControls : public TGenericWindowsTargetPlatformControls<FWinGDKPlatformProperties<IS_CLIENT_ONLY>, FWinGDKTargetDevice>
{	
	typedef FWinGDKPlatformProperties<IS_CLIENT_ONLY> TPlatformProperties;
	typedef TGenericWindowsTargetPlatformControls<TPlatformProperties, FWinGDKTargetDevice> TSuper;

public:

	FWinGDKTargetPlatformControls(ITargetPlatformSettings* TargetPlatformSettings) : TSuper(TargetPlatformSettings)
	{
#if PLATFORM_WINDOWS
		for (const FString& DeviceName : FRemoteWindowsDevice::GetAllRemoteDeviceNames())
		{
			RemoteDevices.Add(MakeShared<FWinGDKRemoteTargetDevice>(*this, DeviceName));
		}
#endif
	}

	virtual ~FWinGDKTargetPlatformControls() = default;

	virtual void GetAllDevices( TArray<ITargetDevicePtr>& OutDevices ) const override
	{
		TSuper::GetAllDevices(OutDevices);
		OutDevices.Append(RemoteDevices);
	}

	TArray<ITargetDevicePtr> RemoteDevices;

public:

	virtual FString CookingDeviceProfileName() const override
	{
		// result must match WINDOWS_OVERRIDE_DEVICEPROFILE_NAME
		return TPlatformProperties::TBaseWindowsPlatformProperties::PlatformName();
	}
};
