// Copyright Epic Games, Inc. All Rights Reserved.

#include "CoreMinimal.h"
#include "WinGDKTargetPlatformControls.h"
#include "Modules/ModuleManager.h"
#include "Interfaces/ITargetPlatformControlsModule.h"
#include "IWinGDKTargetPlatformSettingsModule.h"

#define LOCTEXT_NAMESPACE "FWinGDKTargetPlatformControlsModule"


/**
 * Module for the WinGDK target platform controls.
 */
class FWinGDKTargetPlatformControlsModule : public ITargetPlatformControlsModule
{
public:

	virtual void GetTargetPlatformControls(TArray<ITargetPlatformControls*>& TargetPlatforms, FName& PlatformSettingsModuleName) override
	{
		if (FWinGDKTargetPlatformControls<false>::IsUsable())
		{
			TMap<FString, ITargetPlatformSettings*> PlatformNameToPlatformSettings;
			IWinGDKTargetPlatformSettingsModule* ModuleSettings = FModuleManager::GetModulePtr<IWinGDKTargetPlatformSettingsModule>(PlatformSettingsModuleName);
			if (ModuleSettings != nullptr)
			{
				TMap<FString, ITargetPlatformSettings*> OutMap;
				ModuleSettings->GetPlatformSettingsMaps(OutMap);
				ITargetPlatformControls* GameTP = new FWinGDKTargetPlatformControls<false>(OutMap[FWinGDKPlatformProperties<false>::PlatformName()]);
				ITargetPlatformControls* ClientTP = new FWinGDKTargetPlatformControls<true>(OutMap[FWinGDKPlatformProperties<true>::PlatformName()]);
				TargetPlatforms.Add(GameTP);
				TargetPlatforms.Add(ClientTP);
			}
		}
	}
};


#undef LOCTEXT_NAMESPACE


IMPLEMENT_MODULE( FWinGDKTargetPlatformControlsModule, WinGDKTargetPlatformControls);
