// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class WinGDKTargetPlatformControls : ModuleRules
{
	public WinGDKTargetPlatformControls(ReadOnlyTargetRules Target) : base(Target)
	{
		BinariesSubFolder = "WinGDK";
		SDKVersionRelevantPlatforms.Add(UnrealTargetPlatform.WinGDK);

		// We need a short name here since this can run afoul very easily of the `MAX_PATH` limit when
		// combined with building other targets that make use of this.
		ShortName = "WinGDKTPCon";

		PrivateDependencyModuleNames.AddRange(
			new string[] {
				"Core",
				"DesktopPlatform",
				"RHI",
				"TargetPlatform",
				"WinGDKTargetPlatformSettings",
			}
		);

		PrivateIncludePathModuleNames.AddRange(
			new string[] {
				"WindowsTargetPlatformControls",
				"WindowsTargetPlatformSettings",
				// Needed for WinGDKPlatformProperties.h
				"Core_WinGDK",
			}
		);

		// compile with Engine
		if (Target.bCompileAgainstEngine)
		{
			PrivateDependencyModuleNames.AddRange(
				new string[] {
					"Engine",
				}
			);

			PrivateIncludePathModuleNames.Add("TextureCompressor");
		}
	}
}
