// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "GenericWindowsTargetPlatformSettings.h"
#include "WinGDKPlatformProperties.h"

/**
 * Implements the WinGDK target platform settings.
 */
template<bool IS_CLIENT_ONLY>
class FWinGDKTargetPlatformSettings : public TGenericWindowsTargetPlatformSettings<FWinGDKPlatformProperties<IS_CLIENT_ONLY>>
{	
	typedef FWinGDKPlatformProperties<IS_CLIENT_ONLY> TPlatformProperties;
	typedef TGenericWindowsTargetPlatformSettings<TPlatformProperties> TSuper;

public:

	FWinGDKTargetPlatformSettings() : TSuper()
	{
	}

	virtual ~FWinGDKTargetPlatformSettings() = default;

public:

	virtual bool SupportsFeature( ETargetPlatformFeatures Feature ) const
	{
		switch (Feature)
		{
		case ETargetPlatformFeatures::MultipleGameInstances:
		case ETargetPlatformFeatures::Packaging:
			return true;
		default:
			return TSuper::SupportsFeature(Feature);
		}
	}

	virtual void GetPossibleArchitectures(TArray<FString>& OutArchitectures) const override
	{
		// override the defaults in TGenericWindowsTargetPlatformSettings
		// note: WinGDK on arm64ec may work but is untested at present.
	}
};
