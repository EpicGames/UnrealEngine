// Copyright Epic Games, Inc. All Rights Reserved.

#include "CoreMinimal.h"
#include "WinGDKTargetPlatformSettings.h"
#include "Modules/ModuleManager.h"
#include "Interfaces/ITargetPlatformSettingsModule.h"
#include "IWinGDKTargetPlatformSettingsModule.h"

#define LOCTEXT_NAMESPACE "FWinGDKTargetPlatformSettingsModule"


/**
 * Module for the WinGDK target platform settings.
 */
class FWinGDKTargetPlatformSettingsModule : public IWinGDKTargetPlatformSettingsModule
{
public:
	virtual void GetPlatformSettingsMaps(TMap<FString, ITargetPlatformSettings*>& OutMap) override
	{
		OutMap = PlatformNameToPlatformSettings;
	}

	virtual void GetTargetPlatformSettings(TArray<ITargetPlatformSettings*>& TargetPlatforms) override
	{
		ITargetPlatformSettings* GameTP = new FWinGDKTargetPlatformSettings<false>();
		TargetPlatforms.Add(GameTP);
		PlatformNameToPlatformSettings.Add(FWinGDKPlatformProperties<false>::PlatformName(), GameTP);

		ITargetPlatformSettings* ClientTP = new FWinGDKTargetPlatformSettings<true>();
		TargetPlatforms.Add(ClientTP);
		PlatformNameToPlatformSettings.Add(FWinGDKPlatformProperties<true>::PlatformName(), ClientTP);
	}

private:
	TMap<FString, ITargetPlatformSettings*> PlatformNameToPlatformSettings;
};


#undef LOCTEXT_NAMESPACE


IMPLEMENT_MODULE( FWinGDKTargetPlatformSettingsModule, WinGDKTargetPlatformSettings);
