// Copyright Epic Games, Inc. All Rights Reserved.

#pragma  once 

/*------------------------------------------------------------------------------------
IWinGDKTargetPlatformSettingsModule interface
------------------------------------------------------------------------------------*/

#include "CoreMinimal.h"
#include "Interfaces/ITargetPlatformSettingsModule.h"

class IWinGDKTargetPlatformSettingsModule : public ITargetPlatformSettingsModule
{
public:
	virtual void GetPlatformSettingsMaps(TMap<FString, ITargetPlatformSettings*>& OutMap) = 0;
};
