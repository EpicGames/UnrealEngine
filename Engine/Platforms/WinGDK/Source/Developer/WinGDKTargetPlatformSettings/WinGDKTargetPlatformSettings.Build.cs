// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class WinGDKTargetPlatformSettings : ModuleRules
{
	public WinGDKTargetPlatformSettings(ReadOnlyTargetRules Target) : base(Target)
	{
		BinariesSubFolder = "WinGDK";
		// We need a short name here since this can run afoul very easily of the `MAX_PATH` limit when
		// combined with building other targets that make use of this.
		ShortName = "WinGDKTPSet";

		PrivateDependencyModuleNames.AddRange(
			new string[] {
				"Core",
				"DesktopPlatform",
				"RHI",
				"TargetPlatform",
			}
		);

		PrivateIncludePathModuleNames.AddRange(
			new string[] {
				"WindowsTargetPlatformSettings",
				// Needed for WinGDKPlatformProperties.h
				"Core_WinGDK",
			}
		);

		// compile with Engine
		if (Target.bCompileAgainstEngine)
		{
			PrivateDependencyModuleNames.AddRange(
				new string[] {
					"Engine",
				}
			);

			PrivateIncludePathModuleNames.Add("TextureCompressor");
		}
	}
}
