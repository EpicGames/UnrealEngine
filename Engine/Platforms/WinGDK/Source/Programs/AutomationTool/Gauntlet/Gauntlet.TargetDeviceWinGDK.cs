// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using AutomationTool;
using UnrealBuildBase;
using UnrealBuildTool;

namespace Gauntlet
{
	/// <summary>
	/// Represents a class capable of creating devices of a specific type
	/// </summary>
	public class WinGDKDeviceFactory : IDeviceFactory, IVirtualLocalDevice, IDefaultDeviceSource
	{
		public bool CanSupportPlatform(UnrealTargetPlatform? Platform)
		{
			return Platform == UnrealTargetPlatform.WinGDK;
		}

		public ITargetDevice CreateDevice(string InRef, string InCachePath, string InParam = null)
		{
			if (RemoteWinSupport.IsRemoteDevice(InRef, UnrealTargetPlatform.WinGDK))
			{
				return new TargetDeviceRemoteWin(InRef, InCachePath, UnrealTargetPlatform.WinGDK);
			}

			return new TargetDeviceWinGDK(InRef, InCachePath);
		}

		public bool CanRunVirtualFromPlatform(UnrealTargetPlatform? Platfrom)
		{
			return Platfrom == UnrealTargetPlatform.Win64;
		}

		public UnrealTargetPlatform? GetPlatform()
		{
			return UnrealTargetPlatform.WinGDK;
		}

		public ITargetDevice[] GetDefaultDevices()
		{
			return new ITargetDevice[] { new TargetDeviceWinGDK(Unreal.MachineName,null) };
		}
	}

	/// <summary>
	/// WinGDK implementation of a device to run applications
	/// </summary>
	public class TargetDeviceWinGDK : TargetDeviceWindows
	{
		public TargetDeviceWinGDK(string InName, string InCachePath)
			:base(InName, InCachePath)
		{
			Platform = UnrealTargetPlatform.WinGDK;

			if (!OperatingSystem.IsWindows())
			{
				throw new AutomationException("Unsupported host");
			}

			if (!MSGamingExports.IsWindowsVersionSupported())
			{
				throw new AutomationException("WinGDK requires at least Windows 10 May 2019 Update (version 1903)"); 
			}

			if (!MSGamingExports.IsWindowsDeveloperMode())
			{
				throw new AutomationException("Windows 10 is not in developer mode.");
			}

			if (!MSGamingExports.IsMSGamingRuntimeInstalled())
			{
				throw new AutomationException("Microsoft.GamingServices is not installed.");
			}
		}
	}
}