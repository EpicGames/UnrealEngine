// Copyright Epic Games, Inc. All Rights Reserved.

using System.Collections.Generic;
using System.Runtime.Versioning;
using UnrealBuildTool;

namespace Gauntlet
{
	/// <summary>
	/// A build source for WinGDK that finds and returns packaged builds
	/// </summary>
	[SupportedOSPlatform("windows")]
	public class WinGDKPackagedBuildSource : GDKPackagedBuildSource
	{
		public WinGDKPackagedBuildSource()
			: base( UnrealTargetPlatform.WinGDK, "msixvc", "WinGDK*", "WinGDKPackagedBuildSource")
		{
		}

		/// <inheritdoc/>
		public override List<IBuild> GetBuildsAtPath(string InProjectName, string InPath, int MaxRecursion = 3)
		{
			return GetBuildsAtPath<MSGameStorePackagedBuild>(InProjectName, InPath, MaxRecursion);
		}
	}

	/// <summary>
	/// Staged build source for WinGDK
	/// </summary>
	public class WinGDKBuildSource : StagedBuildSource<StagedBuild>
	{
		public override string BuildName { get { return "WinGDKStagedBuild"; } }

		public override UnrealTargetPlatform Platform { get { return UnrealTargetPlatform.WinGDK; } }

		public override string PlatformFolderPrefix { get { return "WinGDK"; } }
	}
}
