// Copyright Epic Games, Inc. All Rights Reserved.

using AutomationTool;
using System.IO;
using UnrealBuildTool;

namespace Gauntlet
{
	internal class WinGDKPGOPlatform : WinBasePGOPlatform
	{
		public WinGDKPGOPlatform() : base(UnrealTargetPlatform.WinGDK)
		{
		}

		protected override string GetLocalCachePath(ITargetDevice Device)
		{
			using (TargetDeviceWinGDK WinGDKDevice = Device as TargetDeviceWinGDK)
			{
				return WinGDKDevice.LocalCachePath;
			}

			throw new AutomationException($"{Device} is a {Device.GetType()} - expecting TargetDeviceWinGDK");
		}
	}
}



