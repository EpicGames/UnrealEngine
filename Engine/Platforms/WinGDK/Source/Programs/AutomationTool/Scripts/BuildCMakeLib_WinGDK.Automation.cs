// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

class NMakeTargetPlatform_WinGDK : BuildCMakeLib.NMakeTargetPlatform
{
	public override string PlatformOrGroupName => nameof(UnrealTargetPlatform.WinGDK);
	public override string DebugDatabaseExtension => "pdb";
	public override string DynamicLibraryExtension => "dll";
	public override string StaticLibraryExtension => "lib";
	public override bool IsPlatformExtension => true;
}

class MakefileTargetPlatform_WinGDK : BuildCMakeLib.MakefileTargetPlatform
{
	public override string PlatformOrGroupName => nameof(UnrealTargetPlatform.WinGDK);
	public override string DebugDatabaseExtension => "pdb";
	public override string DynamicLibraryExtension => "dll";
	public override string StaticLibraryExtension => "lib";
	public override bool IsPlatformExtension => true;
}

class NinjaBuildTargetPlatform_WinGDK : BuildCMakeLib.NinjaBuildTargetPlatform
{
	public override string PlatformOrGroupName => nameof(UnrealTargetPlatform.WinGDK);
	public override string DebugDatabaseExtension => "pdb";
	public override string DynamicLibraryExtension => "dll";
	public override string StaticLibraryExtension => "lib";
	public override bool IsPlatformExtension => true;
}
