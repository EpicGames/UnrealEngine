// Copyright Epic Games, Inc. All Rights Reserved.
using AutomationTool;
using EpicGames.Core;
using Microsoft.Extensions.Logging;
using System;
using System.Collections.Generic;
using System.IO;
using System.Linq;
using System.Text;
using System.Text.Encodings.Web;
using System.Text.Json;
using System.Text.Json.Nodes;
using UnrealBuildTool;
using JsonObject = System.Text.Json.Nodes.JsonObject;

[Help("Migrate legacy WinGDK platform configuration to Win64 + MSGaming plugins")]
[Help("Project", "Path to project")]
[Help("P4", "Create a changelist for the new files")]
[Help("NoP4", "Change files locally - do not create a perforce changelist")]
[Help("CL", "Override the changelist #")]
class MigrateWinGDK : BuildCommand
{
	FileReference ProjectFile;
	List<string> NewFiles = new List<string>();
	List<string> ModifiedFiles = new List<string>();
	List<string> WritableFiles = new List<string>(); // for testing only
	List<string> UnmigratableDirs = new List<string>();
	List<string> UnmigratedFiles = new List<string>();
	List<string> UnmigratedConfigLines = new List<string>();
	List<string> FilesNotInDepot = new List<string>();
	
	string Project;
	bool bOverwriteExistingFile;
	bool bIsTest;
	int CL = -1;
	string SourceTarget = null;
	string NewTarget = null;
	FileReference SourceTargetScript = null;
	FileReference NewTargetScript = null;
	FileReference NewCustomConfigFile = null;
	
	bool bUseCustomConfig = false;
	bool bUseMSGameOSSSelectorOnly = false;
	bool bWithOSS = true;
	bool bWithChunkInstall = true;
	bool bWithVirtualKeyboard = true;
	bool bWithSaveGameSystem = true;

	ProjectProperties ProjectProperties = null;

	public override void ExecuteBuild()
	{
		// Parse the parameters
		Project = ParseParamValue("Project", "");
		bOverwriteExistingFile = ParseParam("AllowOverwrite");
		CL = ParseParamInt("CL", -1 );

		// sanity check
		if (string.IsNullOrEmpty(Project))
		{
			Logger.LogError("Missing -Project=path/to/project.uproject");
			return;
		}

		ProjectFile = string.IsNullOrEmpty(Project) ? null : ProjectUtils.FindProjectFileFromName(Project);
		if (!FileReference.Exists(ProjectFile))
		{
			Logger.LogError("{ProjectFile} not found", ProjectFile);
			return;
		}

		// cannot have both -P4 and -DebugTest
		bIsTest = ParseParam("DebugTest") && System.Diagnostics.Debugger.IsAttached; //NB. -DebugTest is for debugging this program only
		if (CommandUtils.P4Enabled && bIsTest)
		{
			Logger.LogError("Cannot specify both -P4 and -DebugTest");
			return;
		}

		// cannot have -CL without -P4
		if (!CommandUtils.P4Enabled && CL >= 0)
		{
			Logger.LogError("-CL requires -P4");
			return;
		}

		// sanity check changelist
		if (CommandUtils.P4Enabled && CL >= 0)
		{
			bool bPending;
			if (!P4.ChangeExists(CL, out bPending) || !bPending)
			{
				Logger.LogError("Changelist {CL} cannot be used or is not valid", CL);
				return;
			}
		}

		// most people will want perforce support, but not all studios may be using it day-to-day so best not to use [RequireP4]... enforce explicit opt-in/opt-out instead
		if (!bIsTest && !CommandUtils.P4Enabled && !ParseParam("NoP4"))
		{
			Logger.LogError("Please specify either -P4 or -NoP4 to indicate whether source control is required");
			return;
		}

		
		// if the source project is not in perforce then we are unlikely to be able to add new files to the depot either (probably someone migrating an external project)
		if (CommandUtils.P4Enabled && !P4.FileExistsInDepot(ProjectFile.FullName,false))
		{
			Logger.LogError("{ProjectFile} is not in the current workspace for client \"{ClientName}\" - please re-run with -NoP4 instead of -P4", ProjectFile, P4Env.Client);
			return;
		}

		// ask the user for the parameters
		if (!ValidateParametersFromUser())
		{
			return;
		}


		// Generate the code
		try
		{
			
			MigrateProject();

			// display final report
			Logger.LogInformation("");
			Logger.LogInformation("");
			if (NewFiles.Any() || ModifiedFiles.Any())
			{
				Log.Logger.LogInformation("");
				if(CL > 0)
				{
					Logger.LogInformation("The following files should have been added or edited in changelist {CL}:", CL);
				}
				else
				{
					Logger.LogInformation("The following files should have been created or modified:");
				}

				foreach (string NewFile in NewFiles)
				{
					Logger.LogInformation("\t{NewFile} {Action}", NewFile, (CL > 0 ? "(added)" : "(new)"));
				}
				foreach (string ModifiedFile in ModifiedFiles)
				{
					Logger.LogInformation("\t{ModifiedFile} {Action}", ModifiedFile, (CL > 0 ? "(edit)" : "(modified)"));
				}
				Logger.LogWarning("*** It is strongly recommended that each file is manually verified! Your project may depend on other files that have not been migrated ***");
				ReportWinGDKReferences( NewFiles.Union(ModifiedFiles) );
			}
			else
			{
				Logger.LogWarning("No WinGDK files were found to migrate");
			}
			if (UnmigratableDirs.Any())
			{
				Logger.LogInformation("");
				Logger.LogWarning("The following folders could not be automatically migrated and will likely require manual intervention:");
				foreach (string UnmigratableDir in UnmigratableDirs)
				{
					Logger.LogWarning("\t{UnmigratableDir}", UnmigratableDir);
				}
			}
			if (UnmigratedFiles.Any())
			{
				Logger.LogInformation("");
				Logger.LogWarning("The following files could not be automatically migrated and will likely require manual intervention:");
				foreach (string UnmigratableFile in UnmigratedFiles)
				{
					Logger.LogWarning("\t{UnmigratableFile}", UnmigratableFile);
				}
			}
			if (UnmigratedConfigLines.Any())
			{
					Logger.LogInformation("");
					Logger.LogWarning("The following .ini settings were not not automatically migrated:");
					foreach (string UnmigratedConfigLine in UnmigratedConfigLines)
					{
						Logger.LogWarning("\t{UnmigratedConfigLine}", UnmigratedConfigLine);
					}
			}
			if (FilesNotInDepot.Any())
			{
				Logger.LogInformation("");
				Logger.LogInformation("The following files were ignored during migration because they are not checked into perforce for client \"{ClientName}\":", P4Env.Client);
				foreach( string FileNotInDepot in FilesNotInDepot)
				{
					Logger.LogInformation("\t{FileNotInDepot}", FileNotInDepot);
				}
			}
			if (!bIsTest && WritableFiles.Any())
			{
				Logger.LogInformation("");
				Logger.LogInformation("The following readonly files have been make writable:");
				foreach( string WritableFile in WritableFiles)
				{
					Logger.LogInformation("\t{WritableFile}", WritableFile);
				}
			}
			Logger.LogInformation("");
			Logger.LogInformation("");

			// remove everything if requested (for debugging etc)
			if (bIsTest)
			{
				Logger.LogInformation("Deleting all the newly-created files and restoring any modified files because this is just a debug test run");
				foreach (string NewFile in NewFiles)
				{
					Logger.LogInformation("\t{NewFile} - new file deleted", NewFile);
					File.Delete(NewFile);
				}
				foreach (string WritableFile in WritableFiles)
				{
					File.Delete(WritableFile);
					File.Move(WritableFile + ".tmp.bak", WritableFile);
					File.SetAttributes( WritableFile, File.GetAttributes(WritableFile) | FileAttributes.ReadOnly );
					Logger.LogInformation("\t{WritableFile} - writable file restored", WritableFile);
				}
			}
		}
		catch(Exception)
		{
			// something went wrong - clean up anything we've created so far
			foreach (string NewFile in NewFiles)
			{
				Logger.LogInformation("Removing partial file {NewFile} due to error", NewFile);
				File.Delete(NewFile);
			}
			if (bIsTest)
			{
				foreach (string WritableFile in WritableFiles)
				{
					Logger.LogInformation("Restoring read-only file {WritableFile} due to error", WritableFile);
					File.Delete(WritableFile);
					File.Move(WritableFile + ".tmp.bak", WritableFile);
					File.SetAttributes( WritableFile, File.GetAttributes(WritableFile) | FileAttributes.ReadOnly );
				}
			}
			else
			{
				foreach (string WritableFile in WritableFiles)
				{
					Logger.LogInformation("Read-only file was modified before the error - please restore manually: {WritableFile}", WritableFile);
				}
			}


			// try to safely clean up the perforce changelist too, if it was not specified on the command line
			try
			{
				if (CL > 0 && CommandUtils.P4Enabled && ParseParamInt("CL", -1) != CL)
				{
					Logger.LogInformation("Removing partial changelist ${CL} due to error", CL);
					P4.DeleteChange(CL,true);
				}
			}
			catch(Exception e)
			{
				Logger.LogError(e, "{Message}", e.Message);
			}


			throw;
		}
	}

	private void MigrateProject()
	{
		// migrate main project directory
		MigrateDir(ProjectFile.Directory, false);

		// migrate any restricted folders too
		DirectoryReference RestrictedBaseDir = DirectoryReference.Combine(ProjectFile.Directory, "Restricted");
		if (DirectoryReference.Exists(RestrictedBaseDir))
		{
			foreach( DirectoryReference RestrictedDir in DirectoryReference.EnumerateDirectories(RestrictedBaseDir))
			{
				MigrateDir(RestrictedDir, true);
			}
		}

		// either create a bespoke build target, modify the main project, or use MSGameOSSSelector only
		if (SourceTarget !=  null)
		{
			CreateBespokeBuildTarget();
		}
		else if (bUseMSGameOSSSelectorOnly)
		{
			AddMSGameOSSSelectorOnly();
			CreateMSGameOSSConfig();
		}
		else
		{
			AddMSGamingPlugins();
		}

		// create a custom config file (bespoke build target path only)
		if (bUseCustomConfig)
		{
			CreateCustomConfig();
		}
	}

	private void MigrateDir( DirectoryReference ParentDir, bool bIsRestricted )
	{
		if (DirectoryReference.Exists(ParentDir))
		{
			MigrateConfig(ParentDir);

			if (!bIsRestricted)
			{
				MigrateFolder(ProjectFile.Directory, "Build/Resources", "Build/Windows/MSGaming/Resources");
			}

			WarnAboutUncopyable(ParentDir);
		}
	}


	private void MigrateConfig(DirectoryReference ParentDir)
	{
		// enumerate all ini files in the WinGDK source directory, copying over any suitable ini files we find
		DirectoryReference SrcDir = DirectoryReference.Combine(ParentDir, "Platforms", "WinGDK", "Config");
		if (!DirectoryReference.Exists(SrcDir))
		{
			return;
		}
		List<FileReference> WinGDKConfigFiles = DirectoryReference.EnumerateFiles(SrcDir, "*.*", SearchOption.AllDirectories).ToList();

		// migrate settings from WinGDKEngine.ini to DefaultEngine.ini
		FileReference SrcEngineIni = FileReference.Combine(SrcDir, "WinGDKEngine.ini");
		FileReference DstEngineIni = FileReference.Combine(ParentDir, "Config", "DefaultEngine.ini");
		if (FileReference.Exists(SrcEngineIni))
		{
			bool bModified = false;

			// migrate all relevent GDK target settings
			string[] GDKTargetSettingsKeys =
			{
				// commented-out items potentially exist for WinGDK but will not be migrated to PC-GDK (these may be deprecated or deemed unsuitable)

				// ** UWinGDKTargetSettings **
				//"ContentId",								// docs recommend not using this but may be important for dev scenarios?
				//"bSubmissionValidatorExpirationError",
				//"bMSAFullTrust",
				//"bUseModFolder",
				//"ModFolder",
				//"bRequireXboxLive",
				//"bUseWER",
				"bAlwaysIncludeDX11Dependency",
				"CustomInstallActions",
				//"bSkipAppIdSuffixForShippingConfig",

				// ** UGDKTargetSettings **
				"ApplicationForegroundColor",
				"ApplicationBackgroundColor",
				"CultureStringResources",
				"PerCultureResources",
				"StageIdOverrides",
				"bUseSimplifiedUserModel",
				"UriActivationProtocols",
				"DefaultDisplayName",
				"PackageName",
				"PublisherName",
				"PublisherDisplayName",
				"StoreId",
				"TitleId",
				"PrimaryServiceConfigId",
				"MSAAppId",
				"ProductId",
				"PackageVersion",
				"bIncludeEngineVersionInPackageVersion",
				"IntelligentInstallChunk_DEPRECATED",
				"CustomChunkMapping",
				"bUseFeaturesAndRecipes",
				"IntelligentDeliveryChunks",
				"IntelligentDeliveryFeatures",
				"IntelligentDeliveryRecipes",
				"DLCPlugins",
				"DLCPackages",
				"DLCChunks",
				"bAutoCreateDLCPackages",

				// code-only items
				"AllowedMissingSymbolFiles",
				"bGenerateInstallBatchFile",
				"bGenerateSinglePackages",
				"bGenerateCombinedPackage",
				"bPrecachePackageLayout",
				"bParallelizePackageGeneration",
				"bNoDebugInfoSkipsSymbolBundling",
				"AdditionalMakePkgArguments",
				//"bPackageNameUseMachineName",
			};
			bModified |= MigrateConfigSection(SrcEngineIni, DstEngineIni, "/Script/WindowsTargetPlatform.WindowsTargetSettings", "/Script/MSGamingSupport.MSGamingSettings", KeysToCopy: GDKTargetSettingsKeys);

			// only migrate Services= entries with GDK references
			bModified |= MigrateConfigSection(SrcEngineIni, DstEngineIni, "OnlineServices.OSSAdapter", KeysToCopy: ["Services"], Filter: X => X.Contains("GDK"));

			// only migrate the one key read by the GDK http implementation
			bModified |= MigrateConfigSection(SrcEngineIni, DstEngineIni, "HTTP", KeysToCopy: ["GDKSecurityCacheExpirationSeconds"]);

			// unless we're creating a separate custom config file (or using MSGameOSSSelector which has its own ini mechanism),
			// migrate any GDK references in global config sections. These are kept disabled for reference only as they may change the behavior of the build
			if (!bUseCustomConfig && !bUseMSGameOSSSelectorOnly)
			{
				bool bMigrateCommentedOut = true;
				bModified |= MigrateAdditionalConfigSections(SrcEngineIni, DstEngineIni, bMigrateCommentedOut);
			}

			// copy cvars
			string[] CVarSections =
			{
				"SystemSettings",
				"ConsoleVariables",
			};
			foreach (string ConfigSection in CVarSections)
			{
				bModified |= MigrateConfigSection(SrcEngineIni, DstEngineIni, ConfigSection, Filter: X =>
					X.StartsWith("GDK.") ||
					X.StartsWith("GDKPlatformTextField.") ||
					X.StartsWith("Online.PlayFab.") ||
					X.StartsWith("xb.")
				);
			}

			// blanket migrate other GDK config sections
			string[] GDKConfigSection =
			[
				// online subsystem
				"OnlineSubsystemGDK",
				"OnlineSubsystemGDK.OnlineAsyncTaskGDKCheckForPackageUpdate",
				"OnlineServices.OSSAdapter.GDK.Auth.QueryExternalAuthToken",
				"GDKTaskTimeouts",

				// online tests
				"OnlineServicesTests GDK",
				"LoginCredentials GDK",
				"TestReporting GDK",

				// playfab
				"PlayFab",
				"/Script/PlayFabParty.PlayFabPartyNetDriver",
			];
			foreach (string ConfigSection in GDKConfigSection)
			{
				bModified |= MigrateConfigSection(SrcEngineIni, DstEngineIni, ConfigSection);
			}

			// mark the file as migrated (regardless of whether we modified the target ini file or not)
			WinGDKConfigFiles.Remove(SrcEngineIni);
		}

		// warn about remaining ini files
		UnmigratedFiles.AddRange(WinGDKConfigFiles.Select(X => X.FullName));


		// add RemapeDirectories entry for Platforms/GDK to WindowsEngine.ini
		// ... should not be be needed now that MSGamingRuntimeCustomStagingHandler removes GDK from the restricted folder names
		//[Staging]
		//+RemapDirectories=(From="Engine/Platforms/GDK", To="Engine")
	}

	private bool MigrateAdditionalConfigSections(FileReference SrcEngineIni, FileReference DstEngineIni, bool bMigrateCommentedOut)
	{
		bool bModified = false;

		// migrate the GDK net driver definitions, but leave them disabled to avoid interfering with any existing Win64 settings
		bModified |= MigrateConfigSection(SrcEngineIni, DstEngineIni, "/Script/Engine.Engine", KeysToCopy: ["NetDriverDefinitions"], bMigrateCommentedOut: bMigrateCommentedOut, Filter: X =>
			X.Contains("GDK") ||
			X.Contains("PlayFabParty.")
		);

		// migrate GDK references in other sections, leaving them disable for reference
		string[] AdditionalSections =
		{
				"StreamingInstall",
				"PlatformFeatures",
				"OnlineSubsystem",
				"OnlineServices",
				"OnlineServicesTests",
			};
		foreach (string ConfigSection in AdditionalSections)
		{
			bModified |= MigrateConfigSection(SrcEngineIni, DstEngineIni, ConfigSection, bMigrateCommentedOut: bMigrateCommentedOut, Filter: X =>
				X.Contains("GDK") ||
				X.Contains("XBL") ||
				X.Contains("Xbox") ||
				X.Contains("MSGameStore")
			);
		}

		return bModified;
	}

	private bool MigrateConfigSection(FileReference SrcFile, FileReference DstFile, string ConfigSection, string[] KeysToCopy = null, Func<string, bool> Filter = null, bool bMigrateCommentedOut = false)
	{
		return MigrateConfigSection(SrcFile, DstFile, ConfigSection, ConfigSection, KeysToCopy, Filter, bMigrateCommentedOut);
	}

	private bool MigrateConfigSection(FileReference SrcFile, FileReference DstFile, string SrcConfigSection, string DstConfigSection, string[] KeysToCopy = null, Func<string, bool> Filter = null, bool bMigrateCommentedOut = false )
	{
		bool bModified = false;

		if (!FileReference.Exists(SrcFile))
		{
			return false;
		}

		// ignore the file if we're using -P4 and the file isn't in the depot
		if (CommandUtils.P4Enabled && !P4.FileExistsInDepot(SrcFile.FullName, false))
		{
			FilesNotInDepot.Add(SrcFile.FullName);
			return false;
		}

		// read the existing settings
		ConfigFile SrcConfigFile = new(SrcFile);
		if (!SrcConfigFile.TryGetSection(SrcConfigSection, out ConfigFileSection SrcConfigFileSection))
		{
			return false;
		}

		// load the target ini file and ensure we have the section we want
		bool bNewSection = false;
		List<string> DstConfigFileLines = FileReference.ReadAllLines(DstFile).ToList();
		int DstInsertIndex = DstConfigFileLines.IndexOf($"[{DstConfigSection}]");
		if (DstInsertIndex == -1)
		{
			if (!string.IsNullOrEmpty(DstConfigFileLines.LastOrDefault()))
			{
				DstConfigFileLines.Add(string.Empty);
			}
			DstConfigFileLines.Add($"[{DstConfigSection}]");
			DstInsertIndex = DstConfigFileLines.Count;
			bNewSection = true;
			bModified = true;
		}
		else
		{
			DstInsertIndex++;
		}

		// also load the destination ini file into a ConfigFile to quickly skip existing items
		ConfigFile DstConfigFile = FileReference.Exists(DstFile) ? new(DstFile) : new();
		ConfigFileSection DstConfigFileSection = DstConfigFile.FindOrAddSection(DstConfigSection);

		// copy over all appropriate properties
		foreach (ConfigLine Line in SrcConfigFileSection.Lines)
		{
			if (KeysToCopy != null && !KeysToCopy.Contains(Line.Key))
			{
				UnmigratedConfigLines.Add($"{SrcConfigSection}:{Line}  | not supported by PC GDK");
				continue;
			}

			if (Filter != null && Filter.Invoke(Line.Value) == false)
			{
				continue;
			}

			string NewValue = Line.Value
				.Replace("wingdk", "windows", StringComparison.InvariantCultureIgnoreCase)
				.Replace("WINGDK", "WINDOWS", StringComparison.InvariantCultureIgnoreCase)
				.Replace("WinGDK", "Windows", StringComparison.InvariantCultureIgnoreCase)
			;

			string NewLine = new ConfigLine(Line.Action, Line.Key, NewValue, true).ToString();
			if (bMigrateCommentedOut)
			{
				NewLine = ";" + NewLine;
			}

			// warn about duplicate properties
			if (!bMigrateCommentedOut && DstConfigFileSection.TryGetLine(Line.Key, out ConfigLine ExistingLine))
			{
				if (ExistingLine.Value != Line.Value)
				{
					UnmigratedConfigLines.Add($"{SrcConfigSection}:{Line}  | conflicting value already exists : {ExistingLine.Value}");
				}
				continue;
			}

			// don't add the same commented out item twice
			if (bMigrateCommentedOut && DstConfigFileLines.Contains(NewLine))
			{
				continue;
			}

			DstConfigFileLines.Insert(DstInsertIndex++, NewLine);
			bModified = true;
		}
		if (bNewSection)
		{
			DstConfigFileLines.Insert(DstInsertIndex++, string.Empty);
			bModified = true;
		}

		if (bModified)
		{
			// check out (or add?!) the destination file & get the config section
			if (!AddOrEditFile(DstFile.FullName))
			{
				return false;
			}

			// save the config file
			FileReference.WriteAllLines(DstFile, [.. DstConfigFileLines]);
		}

		return bModified;
	}

	private void MigrateFolder( DirectoryReference ParentDir, string SourceSubfolderName, string DestSubfolderName )
	{
		// simply enumerate and copy all files from the WinGDK directory
		DirectoryReference SrcDir = DirectoryReference.Combine( ParentDir, "Platforms", "WinGDK", SourceSubfolderName );
		DirectoryReference DstDir = DirectoryReference.Combine( ParentDir, DestSubfolderName );
		if (!DirectoryReference.Exists(SrcDir))
		{
			return;
		}

		foreach ( FileReference SrcFile in DirectoryReference.EnumerateFiles(SrcDir, "*.*", SearchOption.AllDirectories ) )
		{
			string SourceFile = SrcFile.FullName;
			string DestFile = Path.Combine(DstDir.FullName, SrcFile.MakeRelativeTo(SrcDir) );

			// ignore the file if we're using -P4 and the file isn't in the depot
			if (CommandUtils.P4Enabled && !P4.FileExistsInDepot(SourceFile, false))
			{
				FilesNotInDepot.Add(SourceFile);
				continue;
			}

			CopyFile( SourceFile, DestFile );
		}
	}

	private void MigrateFolder( DirectoryReference ParentDir, string SubfolderName )
	{
		MigrateFolder(ParentDir, SubfolderName, SubfolderName);
	}

	private void AddMSGamingPlugins()
	{
		string ProjectFileText = FileReference.ReadAllText(ProjectFile);
		JsonNode ProjectNode = JsonNode.Parse(ProjectFileText);

		if (ProjectNode is JsonObject ProjectRoot)
		{
			bool bModified = false;

			string[] GDKPlatforms = GetMicrosoftPlatforms();

			JsonArray Plugins = ProjectRoot["Plugins"] as JsonArray;
			if (Plugins == null)
			{
				Plugins = new JsonArray();
				ProjectRoot["Plugins"] = Plugins;
			}

			bModified |= AddPlugin(Plugins, "MSGameStore", ["Win64"]);
			bModified |= AddPlugin(Plugins, "MSGamingRuntime", ["Win64"]);

			if (bWithOSS)
			{
				bModified |= AddPlugin(Plugins, "OnlineSubsystemGDK", GDKPlatforms);
			}
			if (bWithChunkInstall)
			{
				bModified |= AddPlugin(Plugins, "GDKPackageChunkInstall", GDKPlatforms);
			}
			if (bWithVirtualKeyboard)
			{
				bModified |= AddPlugin(Plugins, "GDKVirtualKeyboard", GDKPlatforms);
			}

			if (bModified)
			{
				EditFile(ProjectFile.FullName);

				string NewProjectFileText = ToJsonString(ProjectRoot);
				FileReference.WriteAllText(ProjectFile, NewProjectFileText);
			}
		}
		else
		{
			UnmigratedFiles.Add(ProjectFile.FullName);
			Logger.LogWarning("Could not read {Project} for migration", ProjectFile.FullName);
		}
	}


	private void AddMSGameOSSSelectorOnly()
	{
		string ProjectFileText = FileReference.ReadAllText(ProjectFile);
		JsonNode ProjectNode = JsonNode.Parse(ProjectFileText);

		if (ProjectNode is JsonObject ProjectRoot)
		{
			bool bModified = false;

			string[] GDKPlatforms = GetMicrosoftPlatforms();

			JsonArray Plugins = ProjectRoot["Plugins"] as JsonArray;
			if (Plugins == null)
			{
				Plugins = new JsonArray();
				ProjectRoot["Plugins"] = Plugins;
			}

			bModified |= AddPlugin(Plugins, "MSGameStore", ["Win64"]);
			bModified |= AddPlugin(Plugins, "MSGamingRuntime", ["Win64"]);
			bModified |= AddPlugin(Plugins, "MSGameOSSSelector", ["Win64"]);
			if (bWithChunkInstall)
			{
				bModified |= AddPlugin(Plugins, "GDKPackageChunkInstall", GDKPlatforms);
			}

			if (bModified)
			{
				EditFile(ProjectFile.FullName);

				string NewProjectFileText = ToJsonString(ProjectRoot);
				FileReference.WriteAllText(ProjectFile, NewProjectFileText);
			}
		}
		else
		{
			UnmigratedFiles.Add(ProjectFile.FullName);
			Logger.LogWarning("Could not read {Project} for migration", ProjectFile.FullName);
		}
	}


	private static bool AddPlugin( JsonArray Plugins, string Name, string[] SupportedPlatormNames = null)
	{
		if (Plugins.OfType<JsonObject>().Any( X => X["Name"]?.ToString() == Name)) 
		{
			return false;
		}

		JsonObject Plugin = new JsonObject
		{
			["Name"] = Name,
			["Enabled"] = true,
		};
		if (SupportedPlatormNames != null)
		{
			JsonArray SupportedPlatforms = new JsonArray();
			foreach (string PlatformName in SupportedPlatormNames)
			{
				SupportedPlatforms.Add(PlatformName);
			}

			Plugin.Add("SupportedTargetPlatforms", SupportedPlatforms);
		}

		Plugins.Add(Plugin);
		return true;
	}


	private void CreateBespokeBuildTarget()
	{
		if (!FileReference.Exists(SourceTargetScript))
		{
			Logger.LogWarning("Cannot create custom build target - SourceTarget not found: {file}", SourceTargetScript);
			return;
		}

		if (FileReference.Exists(NewTargetScript))
		{
			Logger.LogWarning("Cannot create custom build target - NewTarget already exists: {file}", NewTargetScript);
			UnmigratedFiles.Add(SourceTargetScript.FullName);
			return;
		}

		List<string> EnablePlugins =
		[
			"MSGameStore",
			"MSGamingRuntime",
		];
		if (bWithOSS)
		{
			EnablePlugins.Add("OnlineSubsystemGDK");
		}
		if (bWithChunkInstall)
		{
			EnablePlugins.Add("GDKPackageChunkInstall");
		}
		if (bWithVirtualKeyboard)
		{
			EnablePlugins.Add("GDKVirtualKeyboard");
		}


		StringBuilder Lines = new();
		Lines.AppendLine("// Copyright Epic Games, Inc. All Rights Reserved.");
		Lines.AppendLine();
		Lines.AppendLine("using UnrealBuildTool;");
		Lines.AppendLine();
		Lines.AppendLine($"[SupportedPlatforms(\"Win64\")] // PC-GDK is Win64 only. Other platforms should continue to use the {SourceTarget} target");
		Lines.AppendLine($"public class {NewTarget}Target : {SourceTarget}Target");
		Lines.AppendLine("{");
		Lines.AppendLine($"\tpublic {NewTarget}Target(TargetInfo Target) : base(Target)");
		Lines.AppendLine("\t{");

		if (EnablePlugins.Count > 0)
		{
			Lines.AppendLine("\t\tEnablePlugins.AddRange([");
			foreach (string PluginName in EnablePlugins)
			{
				Lines.AppendLine($"\t\t\t\"{PluginName}\",");
			}
			Lines.AppendLine("\t\t]);");
		}

		if (bUseCustomConfig)
		{
			Lines.AppendLine();
			Lines.AppendLine("\t\tBuildEnvironment = TargetBuildEnvironment.Unique;");
			Lines.AppendLine("\t\tCustomConfig = \"MSGameStore\";");
		}

		Lines.AppendLine("\t}");
		Lines.AppendLine("}");
		Lines.AppendLine();

		WriteFile(NewTargetScript.FullName, Lines.ToString());
	}

	
	private void CreateCustomConfig()
	{
		if (FileReference.Exists(NewCustomConfigFile))
		{
			Logger.LogWarning("Cannot create custom config - file already exists: {file}", NewCustomConfigFile);
			return;
		}

		StringBuilder Lines = new();

		if (bWithChunkInstall)
		{
			Lines.AppendLine("[StreamingInstall]");
			Lines.AppendLine("DefaultProviderName=GDKPackageChunkInstall");
			Lines.AppendLine();
		}
		if (bWithSaveGameSystem)
		{
			Lines.AppendLine("[PlatformFeatures]");
			Lines.AppendLine("SaveGameSystemModule=GDKSaveGameSystem");
			Lines.AppendLine();
		}
		if (bWithOSS)
		{
			Lines.AppendLine("[OnlineSubsystem]");
			Lines.AppendLine("DefaultPlatformService=GDK");
			Lines.AppendLine("NativePlatformService=GDK");
			Lines.AppendLine();
		}

		WriteFile(NewCustomConfigFile.FullName, Lines.ToString());


		// migrate other ini settings over from WinGDK to this custom config file that affect things globally
		FileReference SrcEngineIni = FileReference.Combine(ProjectFile.Directory, "Platforms", "WinGDK", "Custom", "WinGDKEngine.ini");
		if (FileReference.Exists(SrcEngineIni))
		{
			bool bMigrateCommentedOut = false;
			MigrateAdditionalConfigSections(SrcEngineIni, NewCustomConfigFile, bMigrateCommentedOut);
		}
	}


	private void CreateMSGameOSSConfig()
	{
		string ProjectName = ProjectFile.GetFileNameWithoutAnyExtensions();

		// Ensure the MSGameOSSSelectorModule is loaded before the OnlineSubsystem is chosen and initialized
		FileReference WindowsEngineIni = FileReference.Combine(ProjectFile.Directory, "Config", "Windows", "WindowsEngine.ini");
		EnsureConfigValue(WindowsEngineIni, "OnlineSubsystem", "+AdditionalModulesToLoad=MSGameOSSSelector");

		// Configure other settings that will only be applied when running as an installed package (or with -ForceOSSGDK)
		if (bWithChunkInstall || bWithSaveGameSystem)
		{
			FileReference MSGameOSSEngineIni = FileReference.Combine(ProjectFile.Directory, "Config", "Windows", "MSGameOSS", "WindowsEngine.ini");
			if (bWithChunkInstall)
			{
				EnsureConfigValue(MSGameOSSEngineIni, "StreamingInstall", "DefaultProviderName=GDKPackageChunkInstall");
			}
			if (bWithSaveGameSystem)
			{
				EnsureConfigValue(MSGameOSSEngineIni, "PlatformFeatures", "SaveGameSystemModule=GDKSaveGameSystem");
			}

			// Required for staging: tells the packager to include the MSGameOSS ini files
			FileReference WindowsGameIni = FileReference.Combine(ProjectFile.Directory, "Config", "Windows", "WindowsGame.ini");
			EnsureConfigValue(WindowsGameIni, "Staging", $"+AllowedConfigFiles={ProjectName}/Config/Windows/MSGameOSS/WindowsEngine.ini");
		}
	}


	private void WarnAboutUncopyable( DirectoryReference ParentDir )
	{
		DirectoryReference SrcDir = DirectoryReference.Combine( ParentDir, "Platforms", "WinGDK" );
		if (!DirectoryReference.Exists(SrcDir))
		{
			return;
		}

		foreach( DirectoryReference ChildDir in DirectoryReference.EnumerateDirectories(SrcDir, "*", SearchOption.TopDirectoryOnly) )
		{
			// skip known directory names that have already been processed by the functions above
			if (ChildDir.GetDirectoryName().Equals("Build", StringComparison.InvariantCultureIgnoreCase ) || 
				ChildDir.GetDirectoryName().Equals("Source", StringComparison.InvariantCultureIgnoreCase) ||
				ChildDir.GetDirectoryName().Equals("Config", StringComparison.InvariantCultureIgnoreCase) ||
				ChildDir.GetDirectoryName().Equals("Plugins", StringComparison.InvariantCultureIgnoreCase))
			{
				continue;
			}

			UnmigratableDirs.Add(ChildDir.FullName);
		}
	}



	/// <summary>
	/// Lists all WinGDK string references in the given source files
	/// </summary>
	/// <param name="SourceFiles"></param>
	private void ReportWinGDKReferences(IEnumerable<string> SourceFiles)
	{
		// only search in these file types
		string[] AllowedFileExtensions = 
		{
			".log",
			".ini",
			".json",
			".txt",
			".uplugin",
			".cs",
			".h",
			".hpp",
			".c",
			".cpp",
			".cxx",
			".inl",
			".ush",
			".usf",
			".pl",
			".py",
			".bat",
		};

		// collect all files with WinGDK references
		Dictionary<string, IEnumerable<string>> WinGDKReferences = new Dictionary<string, IEnumerable<string>>();
		foreach (string SourceFile in SourceFiles)
		{
			if (AllowedFileExtensions.Contains( Path.GetExtension(SourceFile), StringComparer.InvariantCultureIgnoreCase) )
			{
				IEnumerable<string> Lines = File.ReadAllLines(SourceFile).Where(X => X.Contains("WinGDK", StringComparison.InvariantCultureIgnoreCase));
				if (Lines.Any())
				{
					WinGDKReferences.Add(SourceFile, Lines);
				}
			}
		}

		// report them
		if (WinGDKReferences.Any())
		{
			Logger.LogInformation("");
			Logger.LogWarning("The following text files have unmigrated WinGDK references that may require manual intervention:");
			foreach (var WinGDKReference in WinGDKReferences)
			{
				Logger.LogInformation("{Arg0}:", WinGDKReference.Key);
				foreach (string Line in WinGDKReference.Value)
				{
					Logger.LogInformation("\t{Line}", Line);
				}
			}
		}
	}


	#region user querying

	private string ReadInput(string Prompt, string Default = null)
	{
		Console.WriteLine();
		Console.WriteLine(Prompt);
		if (!string.IsNullOrEmpty(Default))
		{
			Console.WriteLine($"[Default: {Default}]");
		}

		string Response = Console.ReadLine();
		if (string.IsNullOrWhiteSpace(Response))
		{
			Response = Default;
		}

		return Response;
	}

	private int ReadInputIndex(string Prompt, string[] Options, int DefaultIndex = -1)
	{
		StringBuilder FullPrompt = new();
		FullPrompt.AppendLine(Prompt);
		for (int Index = 0; Index < Options.Length; Index++)
		{
			FullPrompt.AppendLine($"\t[{Index}] : {Options[Index]} {(Index == DefaultIndex ? " (default)" : "")}");
		}

		while (true)
		{
			string ResultString = ReadInput(FullPrompt.ToString(), (DefaultIndex >= 0) ? DefaultIndex.ToString() : null );		
			if (int.TryParse(ResultString, out int Result) && Result >= 0 && Result < Options.Length)
			{
				return Result;
			}

			Console.WriteLine("invalid choice");
		}
	}

	private string ReadArrayInput(string Prompt, string[] Options, int DefaultIndex = -1)
	{
		return Options[ReadInputIndex(Prompt, Options, DefaultIndex)];
	}

	private bool ReadInputConfirmation(string Prompt, bool bDefaultValue = false)
	{
		string FullPrompt = $"{Prompt} [{(bDefaultValue ? 'Y' : 'y')}/{(bDefaultValue ? 'n' : 'N')}]";
		string Result = ReadInput(FullPrompt, bDefaultValue ? "Y" : "N");
		if (string.IsNullOrEmpty(Result))
		{
			return bDefaultValue;
		}
		else
		{
			return string.Equals( Result.Trim(), "Y", StringComparison.OrdinalIgnoreCase );
		}
	}


	private bool ValidateParametersFromUser()
	{
		// make sure there is a source folder
		DirectoryReference SourceDir = DirectoryReference.Combine(ProjectFile.Directory, "Source");
		if (!DirectoryReference.Exists(SourceDir))
		{
			Logger.LogError("No Source folder in the project - WinGDK migration is only available for non-blueprint projects");
			return false;
		}

		// read project properties
		ProjectProperties = ProjectUtils.GetProjectProperties(ProjectFile, [UnrealTargetPlatform.WinGDK], [UnrealTargetConfiguration.Development]);


		// query all build targets & sanity check
		FileReference[] BuildTargetFiles = ProjectProperties.TargetScripts.ToArray();					//DirectoryReference.EnumerateFiles(SourceDir, "*.target.cs").ToArray();
		string[] BuildTargets = ProjectProperties.Targets.Select( X => X.TargetName ).ToArray();		//BuildTargetFiles.Select( X => X.GetFileNameWithoutAnyExtensions() ).ToArray();

		if (SourceTarget != null)
		{
			int TargetIndex = BuildTargets.FindIndex( X => X.Equals(SourceTarget, StringComparison.InvariantCultureIgnoreCase) );
			if (TargetIndex == -1)
			{
				Logger.LogError("Build target {name} is not known", SourceTarget);
				return false;
			}
			else
			{
				SourceTarget = BuildTargets[TargetIndex]; // ensure correct case
				SourceTargetScript = BuildTargetFiles[TargetIndex];
			}
		}
		else
		{
			Console.WriteLine("-----------------------------------------------------------------------------");
			Console.WriteLine();
			Console.WriteLine("There are three ways you can migrate your WinGDK settings. Which you choose will depend on your project, packaging and publishing requirements");
			Console.WriteLine();
			Console.WriteLine("Option [0]: Update Win64 project with the MSGameOSSSelector plugin.");
			Console.WriteLine(" * Recommended when releasing on other PC stores in addition to the Windows Store and want to re-use the same executable.");
			Console.WriteLine(" * adds MSGameStore, MSGamingRuntime and MSGameOSSSelector to your project");
			Console.WriteLine(" * GDK online services are configured via ini files in Config/Windows/MSGameOSS/");
			Console.WriteLine(" * these settings will only take effect when running as a packaged MSIXVC process (or with -ForceOSSGDK on the command line), otherwise the build behaves as it did before");
			Console.WriteLine();
			Console.WriteLine("Option [1]: Create a PC-GDK specific build target.");
			Console.WriteLine(" * Recommended when releasing on other PC stores in addition to the Windows Store and you need write custom code for PC GDK.");
			Console.WriteLine(" * this will be a different .exe file just like WinGDK. Unlike WinGDK, this uses the same cooked data as the default game target");
			Console.WriteLine(" * default subsystems will be set to GDK, such as save games & online subsystem etc.");
			Console.WriteLine(" * a custom ini file will be created that can override options in the default game (this is what sets the default subsystems)");
			Console.WriteLine();
			Console.WriteLine("Option [2]: Upgrade your Win64 project with all PC GDK plugins.");
			Console.WriteLine(" * Recommended when you are only ever releasing on the Windows Store");
			Console.WriteLine(" * this will add your chosen PC GDK plugins to your project and they will be present in your normal Win64 .exe file");
			Console.WriteLine(" * default subsystems will be unchanged and _not_ set to GDK, such as save games & online subsystem etc.");
			Console.WriteLine(" * you can set them to GDK as default by modifying your WindowsEngine.ini, target.cs and uproject file later as needed");
			Console.WriteLine();
			Console.WriteLine("In all cases, your project's DefaultEngine.ini will be modified to add the configuration options for GDK such as TitleId, Publisher name etc. but these will not alter the behavior of the normal Win64 build");
			Console.WriteLine("");
			Console.WriteLine("-----------------------------------------------------------------------------");

			int MigrationChoice = ReadInputIndex("Which migration approach would you like to use?",
			[
				"Update Win64 project with the MSGameOSSSelector plugin",
				"Create a dedicated PC GDK build target",
				"Upgrade Win64 project with all PC GDK plugins",
			], 0);

			if (MigrationChoice == 0)
			{
				bUseMSGameOSSSelectorOnly = true;
			}
			else if (MigrationChoice == 1)
			{
				int DefaultIndex = BuildTargets.FindIndex(X => X.EndsWith("Game"));
				int TargetIndex = ReadInputIndex("Please select the source build target", BuildTargets, DefaultIndex);

				SourceTarget = BuildTargets[TargetIndex];
				SourceTargetScript = BuildTargetFiles[TargetIndex];
			}
		}

		if (SourceTarget != null)
		{
			bUseCustomConfig = true; //ReadInputConfirmation("Create a custom config file with GDK defaults for this target?", true); // always creating this to keep things simpler
			NewCustomConfigFile = FileReference.Combine(ProjectFile.Directory, "Config", "Windows", "Custom", "MSGameStore", "WindowsEngine.ini");
		}

		if (SourceTarget != null && NewTarget == null)
		{
			NewTarget = GenerateDefaultMSGTargetName(SourceTarget);
			NewTargetScript = FileReference.Combine(ProjectFile.Directory, "Source", $"{NewTarget}.target.cs");
		}


		if (!bUseMSGameOSSSelectorOnly)
		{
			bWithOSS = ReadInputConfirmation("Enable the OnlineSubsystemGDK plugin?", true);
		}
		bWithChunkInstall = ReadInputConfirmation("Enable the GDKPackageChunkInstaller plugin? (use this if you are expecting to do intelligent delivery and use on-demand chunks etc)", true);
		bWithSaveGameSystem = ReadInputConfirmation("Configure the GDK save game system? (use this if you are planning on using Xbox cross-platform save games)", true);
		if (!bUseMSGameOSSSelectorOnly)
		{
			bWithVirtualKeyboard = ReadInputConfirmation("Enable the legacy GDKVirtualKeyboard plugin? (recommendation is to use WinVirtualKeyboard instead, which works on non-GDK builds too)", false);
		}


		// summarise what will be done

		Console.WriteLine();
		Console.WriteLine("-----------------------------------------------------------------------------");
		Console.WriteLine($"WinGDK -> PC GDK Migration for {ProjectFile.GetFileNameWithoutAnyExtensions()}:");
		Console.WriteLine();
		if (bUseMSGameOSSSelectorOnly)
		{
			string ProjectName = ProjectFile.GetFileNameWithoutAnyExtensions();
			Console.WriteLine($"\t{ProjectFile.GetFileName()} will be modified and the following plugins enabled:");
			Console.WriteLine("\t\tMSGameStore (for automatic msixvc package generation)");
			Console.WriteLine("\t\tMSGamingRuntime (for access to the GDK runtime)");
			Console.WriteLine("\t\tMSGameOSSSelector (for dynamic GDK online service configuration when packaged)");
			if (bWithChunkInstall)
			{
				Console.WriteLine("\t\tGDKPackageChunkInstall (for managing intelligent delivery, install on demand etc.)");
			}
			Console.WriteLine();
			Console.WriteLine("\tThe following ini files will be created:");
			Console.WriteLine($"\t\t{ProjectName}/Config/Windows/WindowsEngine.ini : adds +AdditionalModulesToLoad=MSGameOSSSelector to [OnlineSubsystem]");
			Console.WriteLine($"\t\t{ProjectName}/Config/Windows/WindowsGame.ini : adds AllowedConfigFiles staging entry for the MSGameOSS ini");
			if (bWithChunkInstall || bWithSaveGameSystem)
			{
				Console.WriteLine($"\t\t{ProjectName}/Config/Windows/MSGameOSS/WindowsEngine.ini : GDK-specific settings loaded only when packaged");
				Console.WriteLine();
				Console.WriteLine("\tThe MSGameOSS/WindowsEngine.ini will:");
				if (bWithChunkInstall)
				{
					Console.WriteLine("\t\tSet the engine's default chunk installer to the GDK (for intelligent delivery etc)");
				}
				if (bWithSaveGameSystem)
				{
					Console.WriteLine("\t\tSet the engine's save game system to the GDK (for Xbox cross-platform saves)");
				}
			}
		}
		else
		{
			if (SourceTarget == null)
			{
				Console.WriteLine($"\t{ProjectFile.GetFileName()} will be modified and the following plugins enabled:");
			}
			else
			{
				Console.WriteLine($"\tA new build target named {NewTargetScript.GetFileName()} will be created with the following plugins enabled:");
			}
			Console.WriteLine("\t\tMSGameStore (for automatic msixvc package generation)");
			Console.WriteLine("\t\tMSGamingRuntime (for access to the GDK runtime)");
			if (bWithOSS)
			{
				Console.WriteLine("\t\tOnlineSubsystemGDK (for accessing xbox services)");
			}
			if (bWithChunkInstall)
			{
				Console.WriteLine("\t\tGDKPackageChunkInstall (for managing intelligent delivery, install on demand etc.)");
			}
			if (bWithVirtualKeyboard)
			{
				Console.WriteLine("\t\tGDKVirtualKeyboard (for on-screen text input when there is no keyboard available)");
			}
		}
		Console.WriteLine();

		if (bUseCustomConfig)
		{
			Console.WriteLine($"\tA new custom ini file will be created in {NewCustomConfigFile.MakeRelativeTo(ProjectFile.Directory)}. This will contain the following settings:");
			if (bWithSaveGameSystem)
			{
				Console.WriteLine("\t\tSet the engine's save game system to the GDK (for Xbox cross-platform saves)");
			}
			if (bWithChunkInstall)
			{
				Console.WriteLine("\t\tSet the engine's default chunk installer to the GDK (for intelligent delivery etc)");
			}
			if (bWithOSS)
			{
				Console.WriteLine("\t\tSet the engine's default online subsystem to use the GDK (for xbox services)");
			}
			Console.WriteLine();
		}

		Console.WriteLine("-----------------------------------------------------------------------------");
		Console.WriteLine("To use the GDK features in your project you can engage with the modules at runtime. e.g. IMSGameStoreModule::Get().IsPackaged() or IGDKRuntimeModule::Get().GetTitleId()");
		if (bUseMSGameOSSSelectorOnly)
		{
			Console.WriteLine("GDK features will be activated automatically when running as a packaged MSIXVC process.");
			Console.WriteLine("You can test GDK features without packaging by passing -ForceOSSGDK on the command line.");
		}
		else
		{
			if (SourceTarget == null)
			{
				Console.WriteLine("To access GDK specific systems from your project you will need to get the module directly, for example: ISaveGameSystemModule* SaveGameSystemModule = FModuleManager::LoadModulePtr<ISaveGameSystemModule>(\"GDKSaveGameSystem\");");
			}
			else
			{
				Console.WriteLine("GDK specific systems like save games will be the default with this build target");
				Console.WriteLine();
				Console.WriteLine("The custom build target will appear in Visual Studio after re-running GenerateProjectFiles.bat");
				Console.WriteLine("To stage and package your custom build target, you will need to pick it in Project Launcher or specify it on a typical UAT command line:");
				Console.WriteLine($"\tRunUAT BuildCookRun -project={ProjectFile.GetFileNameWithoutAnyExtensions()} -target={NewTarget} -build -cook -stage -pak -package -deploy -run");
			}
		}
		Console.WriteLine();

		Console.WriteLine("-----------------------------------------------------------------------------");
		if (!ReadInputConfirmation("Are you ready to start the migration process?"))
		{
			return false;
		}
		Console.WriteLine();
		Console.WriteLine();

		return true;
	}


	
	
	#endregion





	#region boilerplate & helpers


	/// <summary>
	/// Creates the default target name for a Microsoft Game Store specific build target
	/// </summary>
	/// <param name="TargetName"></param>
	/// <returns></returns>
	private static string GenerateDefaultMSGTargetName( string TargetName )
	{
		return TargetName + "MSG";
	}


	/// <summary>
	/// Returns a list of all Microsoft platform names - used for all GDK plugins' supported platforms
	/// </summary>
	/// <returns></returns>
	private static string[] GetMicrosoftPlatforms()
	{
		List<string> Result = [];

		foreach ( UnrealTargetPlatform Platform in UnrealTargetPlatform.GetValidPlatforms().Where( X => X.IsInGroup(UnrealPlatformGroup.Microsoft)))
		{
			Result.Add(Platform.ToString());
		}

		return Result.ToArray();
	}


	/// <summary>
	/// Serliaze the given object to match the standard UE project format
	/// </summary>
	/// <param name="Obj"></param>
	/// <returns></returns>
	private static string ToJsonString(JsonObject Obj)
	{
		// adapted from JsonObject.ToJsonString
		JsonWriterOptions Options = new() { Indented = true, Encoder = JavaScriptEncoder.UnsafeRelaxedJsonEscaping };
		MemoryStream JsonMemoryStream = new();
		using (Utf8JsonWriter writer = new Utf8JsonWriter(JsonMemoryStream, Options))
		{
			Obj.WriteTo(writer);
			writer.Flush();
		}

		// The Utf8JsonWriter doesn't format json the same as we want in UE, we massage the string to meet our standards 
		string jsonString = Encoding.UTF8.GetString(JsonMemoryStream.ToArray());
		string[] lines = jsonString.Split([Environment.NewLine], StringSplitOptions.None);
		StringBuilder jsonStringBuilder = new();
		foreach (string line in lines)
		{
			// Utf8JsonWriter uses 2 spaces for indents, we replace them with tabs here 
			int numLeadingSpaces = line.TakeWhile(x => x == ' ').Count();
			int numLeadingTabs = numLeadingSpaces / 2;
			jsonStringBuilder.Append('\t', numLeadingTabs);
			jsonStringBuilder.AppendLine(line.Substring(numLeadingSpaces));
		}

		return jsonStringBuilder.ToString();
	}



	/// <summary>
	/// Returns whether there is a valid changelist for adding files to
	/// </summary>
	/// <returns></returns>
	private bool HasChangelist()
	{
		if (!CommandUtils.P4Enabled)
		{
			return false;
		}

		if (CL == -1)
		{
			// add the files to perforce if that is available
			string Description = $"[AUTO-GENERATED] WinGDK -> PC GDK project migration for {ProjectFile.GetFileNameWithoutAnyExtensions()}\n\n#nocheckin verify everything has been generated successfully before checking in - only a basic find/replace has been done!\n#nocheckin your project may depend on other files that have not been migrated";

			CL = P4.CreateChange(P4Env.Client, Description );
		}

		return (CL != -1);
	}

	/// <summary>
	/// Adds the given file to the list of new files, optionally adding to the current changelist if applicable
	/// </summary>
	/// <param name="NewFile"></param>
	private void AddNewFile( string NewFile )
	{
		if (ModifiedFiles.Contains(NewFile) || NewFiles.Contains(NewFile))
		{
			return;
		}

		if (HasChangelist())
		{
			P4.Add(CL, CommandUtils.MakePathSafeToUseWithCommandLine(NewFile) );
		}

		NewFiles.Add(NewFile);
	}

	/// <summary>
	/// Attempts to edit the given file, optionally checking it out if applicable. If this is just a test, read-only files are backed up and made writable
	/// </summary>
	/// <param name="ExistingFile"></param>
	/// <returns></returns>
	private bool EditFile( string ExistingFile )
	{
		if (ModifiedFiles.Contains(ExistingFile) || NewFiles.Contains(ExistingFile))
		{
			return true;
		}

		if (HasChangelist())
		{
			P4.Edit(CL, CommandUtils.MakePathSafeToUseWithCommandLine(ExistingFile) );
		}

		if ((File.GetAttributes(ExistingFile) & FileAttributes.ReadOnly) == FileAttributes.ReadOnly)
		{
			if (bIsTest)
			{
				File.Copy(ExistingFile, ExistingFile + ".tmp.bak");
			}

			WritableFiles.Add(ExistingFile);
			File.SetAttributes( ExistingFile, File.GetAttributes(ExistingFile) & ~FileAttributes.ReadOnly );
		}

		ModifiedFiles.Add(ExistingFile);
		return true;
	}

	/// <summary>
	/// Adds the given file to the list of new files, or checks it out if it already exists
	/// </summary>
	/// <param name="TargetFile"></param>
	/// <returns></returns>
	private bool AddOrEditFile(string TargetFile)
	{
		if (!File.Exists(TargetFile))
		{
			AddNewFile(TargetFile);
			return true;
		}

		if (ModifiedFiles.Contains(TargetFile) || NewFiles.Contains(TargetFile))
		{
			return true;
		}

		return EditFile(TargetFile);
	}


	/// <summary>
	/// Copies the given file to a new location, optionally checking it out if applicable.
	/// </summary>
	/// <param name="SourceFile"></param>
	/// <param name="TargetFile"></param>
	/// <returns></returns>
	private bool CopyFile( string SourceFile, string TargetFile )
	{
		if (!File.Exists(SourceFile))
		{
			return false;
		}

		if (ModifiedFiles.Contains(TargetFile) || NewFiles.Contains(TargetFile))
		{
			return true;
		}

		if (File.Exists(TargetFile) && !(bOverwriteExistingFile && EditFile(TargetFile)))
		{
			Logger.LogWarning("Skipping {TargetFile} as it already exists", TargetFile);
			return false;
		}

		string TargetDir = Path.GetDirectoryName(TargetFile);
		Directory.CreateDirectory(TargetDir);
		File.Copy(SourceFile, TargetFile);
		File.SetAttributes(TargetFile, File.GetAttributes(TargetFile) & ~FileAttributes.ReadOnly);

		AddNewFile(TargetFile);
		return true;
	}

	/// <summary>
	/// Writes the given string to a file, optionally checking out if applicable.
	/// </summary>
	/// <param name="TargetFile"></param>
	/// <param name="FileContents"></param>
	/// <returns></returns>
	private bool WriteFile( string TargetFile, string FileContents )
	{
		if (File.Exists(TargetFile) && !(bOverwriteExistingFile && EditFile(TargetFile)))
		{
			Logger.LogWarning("Skipping {TargetFile} as it already exists", TargetFile);
			return false;
		}

		string TargetDir = Path.GetDirectoryName(TargetFile);
		Directory.CreateDirectory(TargetDir);
		File.WriteAllText(TargetFile, FileContents);

		AddNewFile(TargetFile);
		return true;
	}

	/// <summary>
	/// Ensures the given value line is present in the specified ini section of the target file.
	/// Creates the file and/or section if they do not already exist.
	/// </summary>
	/// <param name="TargetFile">The ini file to update</param>
	/// <param name="SectionName">The ini section name (without brackets)</param>
	/// <param name="Line">The exact line to add (e.g. "+Key=Value")</param>
	/// <returns>True if the file was modified</returns>
	private bool EnsureConfigValue(FileReference TargetFile, string SectionName, string Line)
	{
		List<string> FileLines;
		if (FileReference.Exists(TargetFile))
		{
			FileLines = FileReference.ReadAllLines(TargetFile).ToList();
		}
		else
		{
			FileLines = new List<string>();
		}

		// Skip if the exact line already exists in the file
		if (FileLines.Contains(Line))
		{
			return false;
		}

		// Find the section header
		int SectionIndex = FileLines.IndexOf($"[{SectionName}]");
		int InsertIndex;
		bool bNewSection = SectionIndex == -1;

		if (bNewSection)
		{
			if (!string.IsNullOrEmpty(FileLines.LastOrDefault()))
			{
				FileLines.Add(string.Empty);
			}
			FileLines.Add($"[{SectionName}]");
			InsertIndex = FileLines.Count;
		}
		else
		{
			InsertIndex = SectionIndex + 1;
		}

		FileLines.Insert(InsertIndex, Line);
		if (bNewSection)
		{
			FileLines.Add(string.Empty);
		}

		if (!AddOrEditFile(TargetFile.FullName))
		{
			return false;
		}

		Directory.CreateDirectory(TargetFile.Directory.FullName);
		FileReference.WriteAllLines(TargetFile, [.. FileLines]);
		return true;
	}

	#endregion
}
