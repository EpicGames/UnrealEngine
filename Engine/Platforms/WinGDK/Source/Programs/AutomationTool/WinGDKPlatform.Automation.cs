// Copyright Epic Games, Inc. All Rights Reserved.
using System;
using System.Collections.Generic;
using System.Linq;
using System.Text;
using System.IO;
using AutomationTool;
using UnrealBuildTool;
using EpicGames.Core;
using UnrealBuildBase;
using AutomationUtils.GDK;
using Microsoft.Extensions.Logging;


public class WinGDKPlatform : Win64Platform
{
	protected virtual string IniSection_TargetSettings { get { return "/Script/WindowsTargetPlatform.WindowsTargetSettings"; } }

	
	[CustomDeploymentHandlerAttribute("WinGDK")]
	internal class WinGDKCustomDeploymentHandler : MSGameStoreCustomDeploymentHandler
	{
		protected override string IniSection_TargetSettings => "/Script/WindowsTargetPlatform.WindowsTargetSettings";
		internal WinGDKCustomDeploymentHandler(Platform AutomationPlatform, ProjectParams Params, DeploymentContext SC) : base(AutomationPlatform, Params, SC)
		{
		}
		
		public override bool PreGetFilesToDeployOrStage(ProjectParams Params, DeploymentContext SC)
		{
			SC.RestrictedFolderNames.RemoveWhere(x => x == "Win64");
			return base.PreGetFilesToDeployOrStage(Params, SC);
		}
	}




	public WinGDKPlatform()
		: base(UnrealTargetPlatform.WinGDK)
	{
	}

	public override CustomDeploymentHandler GetCustomDeploymentHandler(ProjectParams Params, DeploymentContext SC)
	{
		return new WinGDKCustomDeploymentHandler(this, Params, SC);
	}

	#region Turnkey support

	public override DeviceInfo[] GetDevices()
	{
		List<DeviceInfo> Devices = new List<DeviceInfo>();

		if (HostPlatform.Current.HostEditorPlatform == UnrealTargetPlatform.Win64)
		{
			DeviceInfo LocalMachine = new DeviceInfo(TargetPlatformType);
			LocalMachine.Name = Unreal.MachineName;
			LocalMachine.Id = Unreal.MachineName;
			LocalMachine.Type = "Computer";
			LocalMachine.SoftwareVersion = Environment.OSVersion.Version.ToString();

			Devices.Add(LocalMachine);

			Devices.AddRange(RemoteWinSupport.GetDevices(UnrealTargetPlatform.WinGDK));
		}

		return Devices.ToArray();
	}

	public override string GetSDKCreationHelp()
	{
		IEnumerable<UnrealTargetPlatform> GDKPlatforms = UnrealTargetPlatform.GetValidPlatforms().Where(X => X.IsInGroup(UnrealPlatformGroup.GDK));
		string PlatformList = string.Join(",", GDKPlatforms);

		StringBuilder Out = new StringBuilder();
		Out.AppendLine("GDK installers should already come zipped, so you just need to point Turnkey at them.");
		Out.AppendLine();
		Out.AppendLine("Examples:");
		Out.AppendLine("<FileSource>");
		Out.AppendLine($"  <Platform>{PlatformList}</Platform>");
		Out.AppendLine("  <Version>$(ExpVersion)</Version>");
		Out.AppendLine("  <Name>GDK $(ExpVersion)</Name>");
		Out.AppendLine("  <Type>Full</Type>");
		Out.AppendLine("  <Source HostPlatform=\"Win64\">fileexpansion:googledrive:/SdkInstallers/GDK/SDKs/$[ExpVersion]/$[1]-GDK$[2]-$[3].ZIP</Source>");
		Out.AppendLine("</FileSource>");
		Out.AppendLine();

		return Out.ToString();
	}


	public override bool GetSDKInstallCommand(out string Command, out string Params, ref bool bRequiresPrivilegeElevation, ref bool bCreateWindow, ITurnkeyContext TurnkeyContext)
	{
		string OutputPath = TurnkeyContext.GetVariable("CopyOutputPath");
		Command = OutputPath;

		if (!Command.EndsWith("GDK.exe", StringComparison.InvariantCultureIgnoreCase))
		{
			Command = Path.Combine(Command, "GDK.exe");
		}

		// install everything, quietly
		Params = "/q";

		return true;
	}

	#endregion



	public override string GetEditorCookPlatform()
	{
		return GetCookPlatform(false, false);
	}

	/// <summary>
	/// Gets cook platform name for this platform.
	/// </summary>
	/// <returns>Cook platform string.</returns>
	public override string GetCookPlatform(bool bDedicatedServer, bool bIsClientOnly)
	{
		if (bDedicatedServer)
		{
			throw new AutomationException("WinGDK cannot host dedicated servers.");
		}

		string Return = PlatformType.ToString();
        return bIsClientOnly ? Return + "Client" : Return;
	}


	public override string LocalPathToTargetPath(string LocalPath, string LocalRoot)
	{
		return LocalPath.Replace("\\", "/").Replace(LocalRoot, "../../..");
	}
	
	// used by BuildCommonTools
	public override void MakeAgenda(UnrealBuild.BuildAgenda Agenda, List<string> ExtraBuildProducts)
	{
		Agenda.AddTarget("CrashReportClient", CrashReportPlatform.Value, UnrealBuildTool.UnrealTargetConfiguration.Development);
		Agenda.AddTarget("BootstrapPackagedGame", BootstrapExePlatform.Value, UnrealBuildTool.UnrealTargetConfiguration.Shipping);
	}

	// used by BuildProjectCommand
	public override void PreBuildAgenda(UnrealBuild Build, UnrealBuild.BuildAgenda Agenda, ProjectParams Params)
	{
		if (!Params.NoBootstrapExe && !Unreal.IsEngineInstalled())
		{
			Agenda.AddTarget("BootstrapPackagedGame", BootstrapExePlatform.Value, UnrealBuildTool.UnrealTargetConfiguration.Shipping);
		}
	}


	public override UnrealTargetPlatform? CrashReportPlatform
	{
		get { return UnrealTargetPlatform.Win64; }
	}

	public override UnrealTargetPlatform? BootstrapExePlatform
	{
		get { return UnrealTargetPlatform.Win64; }
	}

	public override bool UseAbsLog
	{
		get { return true; }
	}

	public override bool CanHostPlatform(UnrealTargetPlatform Platform)
	{
		return false;
	}
}
