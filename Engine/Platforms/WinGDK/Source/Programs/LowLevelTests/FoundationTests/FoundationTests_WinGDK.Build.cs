// Copyright Epic Games, Inc. All Rights Reserved.

using System.IO;
using UnrealBuildTool;

public class FoundationTests_WinGDK : FoundationTests
{
	static FoundationTests_WinGDK()
	{
		TestMetadata.SupportedPlatforms.Add(UnrealTargetPlatform.WinGDK);
		TestMetadata.PlatformGauntletArgs.Add(UnrealTargetPlatform.WinGDK, "-captureoutput");
	}

    public FoundationTests_WinGDK(ReadOnlyTargetRules Target) : base(Target)
    {
		
    }
}
