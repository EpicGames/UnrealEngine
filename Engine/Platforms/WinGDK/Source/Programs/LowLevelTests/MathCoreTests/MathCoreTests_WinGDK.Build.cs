// Copyright Epic Games, Inc. All Rights Reserved.

using System.IO;
using UnrealBuildTool;

public class MathCoreTests_WinGDK : MathCoreTests
{
	static MathCoreTests_WinGDK()
	{
		TestMetadata.SupportedPlatforms.Add(UnrealTargetPlatform.WinGDK);
		TestMetadata.PlatformGauntletArgs.Add(UnrealTargetPlatform.WinGDK, "-captureoutput");
	}

    public MathCoreTests_WinGDK(ReadOnlyTargetRules Target) : base(Target)
    {
		
    }
}
