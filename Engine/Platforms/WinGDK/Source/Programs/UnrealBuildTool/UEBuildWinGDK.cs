// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.IO;
using System.Linq;
using System.Runtime.Versioning;
using System.Threading;
using EpicGames.Core;
using Microsoft.Extensions.Logging;
using UnrealBuildBase;
using UnrealBuildTool;

namespace UnrealBuildTool
{
	partial struct UnrealTargetPlatform
	{
		/// <summary>
		/// Adds WinGDK as a target platform identifier
		/// </summary>
		public static UnrealTargetPlatform WinGDK { get; } = FindOrAddByName("WinGDK");
	}

	/// <summary>
	/// Target rules for WinGDK
	/// </summary>
	public class WinGDKTargetRules
	{
		/// <summary>
		/// Whether exceptions should be ultimately passed on to Windows Error Reporting
		/// </summary>
		[ConfigFile(ConfigHierarchyType.Engine, "/Script/WindowsTargetPlatform.WindowsTargetSettings", "bUseWER")]
		public bool bUseWER = true;

		/// <summary>
		/// Whether the embedded manifest will indicate the application is DPI aware
		/// </summary>
		[ConfigFile(ConfigHierarchyType.Engine, "/Script/Engine.UserInterfaceSettings", "bAllowHighDPIInGameMode")]
		public bool bAllowHighDPIInGameMode = false;

	}

	#region target rules boilerplate

	/// <summary>
	/// Read only target rules for WinGDK
	/// </summary>
	public class ReadOnlyWinGDKTargetRules
	{
		private readonly WinGDKTargetRules WinGDKInner;

		internal ReadOnlyWinGDKTargetRules(WinGDKTargetRules Inner)
		{
			WinGDKInner = Inner;
		}

		#region Read-only accessor properties 
#pragma warning disable CS1591
		public bool bUseWER => WinGDKInner.bUseWER;
		public bool bAllowHighDPIInGameMode => WinGDKInner.bAllowHighDPIInGameMode;
#pragma warning restore CS1591
		#endregion
	}

	abstract partial class TargetRules
	{
		/// <summary>
		/// WinGDK-specific target settings.
		/// </summary>
		[ConfigSubObject]
		public WinGDKTargetRules WinGDKPlatform = new();
	}

	public partial class ReadOnlyTargetRules
	{
		private ReadOnlyWinGDKTargetRules? _WinGDKPlatform = null;
		/// <summary>
		/// Singleton accessor for WinGDKPlatform
		/// </summary>
		public ReadOnlyWinGDKTargetRules WinGDKPlatform
		{
			get
			{
				_WinGDKPlatform ??= new ReadOnlyWinGDKTargetRules(Inner.WinGDKPlatform);
				return _WinGDKPlatform;
			}
		}
	}
	#endregion
}

class WinGDKArchitectureConfig : UnrealArchitectureConfig
{
	public WinGDKArchitectureConfig()
		: base(UnrealArchitectureMode.SingleArchitecture, new[] { UnrealArch.X64 })
	{
	}
}

class WinGDKPlatform : WindowsPlatform
{

	public WinGDKPlatform(WinGDKPlatformSDK InSDK, ILogger InLogger)
		: base(UnrealTargetPlatform.WinGDK, InSDK, new WinGDKArchitectureConfig(), InLogger)
	{
	}

	public override string GetPlatformName() => Platform.ToString();

	[SupportedOSPlatform("windows")]
	protected override VCEnvironment CreateVCEnvironment(TargetRules Target)
	{
		// Get required Windows SDK
		VersionNumber WindowsSdkVersion;
		DirectoryReference WindowsSdkDir;
		GDKPlatformSDK.GetGSDKWindowsSDK(Logger, Target.Platform, out WindowsSdkVersion!, out WindowsSdkDir!);

		// Create the GDK VC environment
		VCEnvironmentParameters Params = new(Target.WindowsPlatform.Compiler, Target.WindowsPlatform.ToolChain, Platform, UnrealArch.X64, Target.WindowsPlatform.CompilerVersion, Target.WindowsPlatform.ToolchainVersion, WindowsSdkVersion.ToString(), WindowsSdkDir.ToString(), Target.WindowsPlatform.bUseCPPWinRT, Target.WindowsPlatform.bAllowClangLinker, Target.WindowsPlatform.bAllowRadLinker, Logger);
		return new VCEnvironment(Params, Logger);
	}

	/// <inheritdoc/>
	public override VCProjectFileFormat GetRequiredVisualStudioVersion()
	{
		IEnumerable<VisualStudioInstallation> Installations = MicrosoftPlatformSDK.FindVisualStudioInstallations(Logger);

		Version? version = Installations
			.SelectMany(x => x.Packages.Where(x => x.Key.StartsWith("Component.Microsoft.Gaming.GDK.PC", StringComparison.Ordinal)).Select(x => x.Value.Version))
			.Max();

		if (version != null && version < GetVersionRequiredForVisualStudio(VCProjectFileFormat.VisualStudio2026))
		{
			return VCProjectFileFormat.VisualStudio2022;
		}

		return VCProjectFileFormat.Default;
	}

	/// <inheritdoc/>
	public override Version GetVersionRequiredForVisualStudio(VCProjectFileFormat Format)
	{
		return GDKPlatformSDK.GetVersionRequiredForVisualStudio(Format);
	}

	public override void ResetTarget(TargetRules Target)
	{
		base.ResetTarget(Target);

		// always build the GDK OSS by default
		Target.BuildPlugins.Add("OnlineSubsystemGDK");

		// always build the GDK virtual keyboard plugin by default
		Target.BuildPlugins.Add("GDKVirtualKeyboard");

		// always build the GDK platform chunk installer by default
		Target.BuildPlugins.Add("GDKPackageChunkInstall");

		// we require win10 minimum for GDK 
		Target.WindowsPlatform.TargetWindowsVersion = 0x0A00;

		// deploy by default
		Target.bDeployAfterCompile = true;

		// Use libCurl by default, xCurl is optional on WinGDK
		Target.WindowsPlatform.bUseXCurl = false;
	}

	private static readonly Lock ManifestWriteLock = new();

	public override void ValidateTarget(TargetRules Target)
	{
		base.ValidateTarget(Target);

		Target.WindowsPlatform.bUseBundledDbgHelp = false; //we're going to be on at least Win10 1903 so should not run into the issue. Also prevents SubmissionValidator warnings for missing dbghelp.pdb

		if (Target.bAllowLTCG || Target.bPGOOptimize || Target.bPGOProfile)
		{
			Target.bUseIncrementalLinking = false;
		}

		// load the linker manifest template & update properties
		if (!Target.bGenerateProjectFiles)
		{
			FileReference ManifestTemplateFile = FileReference.Combine(Unreal.EngineDirectory, @"Platforms\WinGDK\Build\Resources", "Default.manifest.template");
			if (Target.ProjectFile != null)
			{
				// check for project-specific override
				FileReference ProjectManifestTemplateFile = FileReference.Combine(Target.ProjectFile.Directory, @"Platforms\WinGDK\Build\Resources", "Default.manifest.template");
				if (FileReference.Exists(ProjectManifestTemplateFile))
				{
					ManifestTemplateFile = ProjectManifestTemplateFile;
				}
			}
			string ManifestTemplate = File.ReadAllText(ManifestTemplateFile.FullName);
			ManifestTemplate = ManifestTemplate.Replace("%DPI_SETTING%", Target.WinGDKPlatform.bAllowHighDPIInGameMode ? "PerMonitorV2" : "unaware", StringComparison.Ordinal);

			// save the generated linker manifest
			FileReference GeneratedManifestFile = FileReference.Combine(Target.ProjectFile?.Directory ?? Unreal.EngineDirectory, @"Intermediate\WinGDK\LinkerManifest", "Generated.manifest");
			DirectoryReference.CreateDirectory(GeneratedManifestFile.Directory);

			lock (ManifestWriteLock)
			{
				// TODO: This is actually not great. This file could look different for multiple targets but all targets will end up using the last written file
				FileReference.WriteAllTextIfDifferent(GeneratedManifestFile, ManifestTemplate);
			}

			Target.WindowsPlatform.ManifestFile = GeneratedManifestFile.FullName;
		}
	}

	/// <summary>
	/// Whether this platform should build a monolithic binary
	/// </summary>
	public override bool ShouldCompileMonolithicBinary(UnrealTargetPlatform InPlatform) => true; // modular build not supported

	public override void ModifyModuleRulesForOtherPlatform(string ModuleName, ModuleRules Rules, ReadOnlyTargetRules Target)
	{
		base.ModifyModuleRulesForOtherPlatform(ModuleName, Rules, Target);

		// set PLATFORM_WINGDK to 0 in modules that WinGDK will set to 1, so that #if checks don't trigger the error of being undefined,
		// and we don't need to set PLATFORM_WINGDK to 0 in Core somewhere
		if (Rules.bAllowConfidentialPlatformDefines && Target.Platform != Platform)
		{
			Rules.PrivateDefinitions.Add("PLATFORM_WINGDK=0");
			Rules.PrivateDefinitions.Add("UE_PLATFORM_WINGDK=0");
		}

		bool bIsPlatformAvailableForTarget = UEBuildPlatform.IsPlatformAvailableForTarget(Platform, Target, bIgnoreSDKCheck: true);
		bool bIsPlatformAvailableForTargetWithSDK = UEBuildPlatform.IsPlatformAvailableForTarget(Platform, Target);

		// don't do any target platform stuff if platform is not available
		if (!bIsPlatformAvailableForTarget)
		{
			return;
		}

		if (Target.Platform == UnrealTargetPlatform.Win64 && Target.Architecture != UnrealArch.Arm64)
		{
			bool bIsMonolithic = true;
			UEBuildPlatform BuildPlatform = GetBuildPlatform(Target.Platform);

			if (BuildPlatform != null)
			{
				bIsMonolithic = BuildPlatform.ShouldCompileMonolithicBinary(Target.Platform);
			}

			if ((Target.Type != TargetType.Editor) || !bIsMonolithic)
			{
				if (!Target.bBuildRequiresCookedData)
				{
					if (ModuleName == "Engine")
					{
						if (Target.bBuildDeveloperTools)
						{
							Rules.DynamicallyLoadedModuleNames.Add("WinGDKTargetPlatformSettings");
							if (bIsPlatformAvailableForTargetWithSDK)
							{
								Rules.DynamicallyLoadedModuleNames.Add("WinGDKTargetPlatform");
								Rules.DynamicallyLoadedModuleNames.Add("WinGDKTargetPlatformControls");
								Rules.DynamicallyLoadedModuleNames.Add("WinGDKPlatformEditor");
							}
						}
					}
				}

				// allow standalone tools to use target platform modules, without needing Engine
				if (ModuleName == "TargetPlatform")
				{
					if (Target.bForceBuildTargetPlatforms)
					{
						Rules.DynamicallyLoadedModuleNames.Add("WinGDKTargetPlatformSettings");
						if (bIsPlatformAvailableForTargetWithSDK)
						{
							Rules.DynamicallyLoadedModuleNames.Add("WinGDKTargetPlatform");
							Rules.DynamicallyLoadedModuleNames.Add("WinGDKTargetPlatformControls");
						}
					}
				}
			}
		}
	}

	public override void ModifyModuleRulesForActivePlatform(string ModuleName, ModuleRules Rules, ReadOnlyTargetRules Target)
	{
		base.ModifyModuleRulesForActivePlatform(ModuleName, Rules, Target);

		Rules.PrivateDefinitions.Add("PLATFORM_WINGDK=1");
		Rules.PrivateDefinitions.Add("UE_PLATFORM_WINGDK=1");
	}

	public override void SetUpEnvironment(ReadOnlyTargetRules Target, CppCompileEnvironment CompileEnvironment, LinkEnvironment LinkEnvironment)
	{
		base.SetUpEnvironment(Target, CompileEnvironment, LinkEnvironment);

		CompileEnvironment.Definitions.Add("PLATFORM_IS_EXTENSION=1");
		CompileEnvironment.Definitions.Add("_GAMING_DESKTOP");

		if (Target.WinGDKPlatform.bUseWER)
		{
			CompileEnvironment.Definitions.Add("WINDOWS_USE_WER=1");
		}
	}

	public override void AddExtraModules(ReadOnlyTargetRules Target, List<string> ExtraModuleNames)
	{
		if (Target.bCompileAgainstEngine)
		{
			ExtraModuleNames.Add("MSGameStorePlatformFeatures");
			ExtraModuleNames.Add("GDKSaveGameSystem");
		}

		ExtraModuleNames.Add("GDKRuntime");

		base.AddExtraModules(Target, ExtraModuleNames);
	}

	public override UEToolChain CreateToolChain(ReadOnlyTargetRules Target)
	{
		WinGDKToolChain toolchain = new(Target, Logger);
		if (Target.StaticAnalyzer == StaticAnalyzer.PVSStudio)
		{
			return new PVSToolChain(Target, toolchain, Logger);
		}
		return toolchain;
	}

	/// <summary>
	/// Allows the platform header name to be overridden to differ from the platform name.
	/// GDK platforms do not utilize an override for the platform header name.
	/// </summary>
	protected override string? GetOverridePlatformHeaderName() => null;
}

class WinGDKPlatformFactory : UEBuildPlatformFactory
{
	public override UnrealTargetPlatform TargetPlatform => UnrealTargetPlatform.WinGDK;

	/// <summary>
	/// Register the platform with the UEBuildPlatform class
	/// </summary>
	public override void RegisterBuildPlatforms(ILogger Logger)
	{
		WinGDKPlatformSDK SDK = new(Logger);

		// Register this build platform for WinGDK
		UEBuildPlatform.RegisterBuildPlatform(new WinGDKPlatform(SDK, Logger), Logger);
		UEBuildPlatform.RegisterPlatformWithGroup(UnrealTargetPlatform.WinGDK, UnrealPlatformGroup.GDK);
		UEBuildPlatform.RegisterPlatformWithGroup(UnrealTargetPlatform.WinGDK, UnrealPlatformGroup.Windows);
		UEBuildPlatform.RegisterPlatformWithGroup(UnrealTargetPlatform.WinGDK, UnrealPlatformGroup.Microsoft);
		UEBuildPlatform.RegisterPlatformWithGroup(UnrealTargetPlatform.WinGDK, UnrealPlatformGroup.Desktop);
	}
}
