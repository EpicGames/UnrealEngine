// Copyright Epic Games, Inc. All Rights Reserved.

using Microsoft.Extensions.Logging;
using System;

namespace UnrealBuildTool.GDK
{
	partial struct GDKGameConfigGeneratorFactory
	{
#pragma warning disable CA1823 // Avoid unused private fields - instantiation is side-effectful, preferable to a static ctor
#pragma warning disable IDE0052
		private static readonly Type GenWinGDK = RegisterGenerator(UnrealTargetPlatform.WinGDK, typeof(WinGDKGameConfigGenerator));
#pragma warning restore CA1823
#pragma warning restore IDE0052
	}

	class WinGDKGameConfigGenerator : MSGamingGameConfigGenerator
	{
		protected override string IniSection_PlatformTargetSettings => "/Script/WindowsTargetPlatform.WindowsTargetSettings";
		protected override string? IniSection_GeneralPlatformSettings => null;
		protected override string IniPlatformName => Platform.ToString();

		/// <summary>
		/// Create a manifest generator for WinGDK
		/// </summary>
		public WinGDKGameConfigGenerator(ILogger InLogger)
			: base(UnrealTargetPlatform.WinGDK, InLogger)
		{
		}
	}
}
