// Copyright Epic Games, Inc. All Rights Reserved.

namespace UnrealBuildTool
{
	//////////////////////////////////////////////////////////////////////////////////////////////
	// If you are looking for version numbers, see Engine/Platforms/WinGDK/Config/WinGDK_SDK.json
	//////////////////////////////////////////////////////////////////////////////////////////////
}
