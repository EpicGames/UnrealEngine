// Copyright Epic Games, Inc. All Rights Reserved.

using EpicGames.Core;
using Microsoft.Extensions.Logging;
using System;
using System.Text.RegularExpressions;

///////////////////////////////////////////////////////////////////
// If you are looking for supported version numbers, look in the
// WinGDKPlatformSDK.Versions.cs file next to this file
///////////////////////////////////////////////////////////////////

namespace UnrealBuildTool
{
	partial class WinGDKPlatformSDK : GDKPlatformSDK
	{
		public WinGDKPlatformSDK(ILogger InLogger)
			: base(InLogger)
		{
		}

		public override bool TryConvertVersionToInt(string? StringValue, out ulong OutValue, string? Hint)
		{
			if (TryConvertVersionToIntInternal(StringValue, out OutValue))
			{
				return true;
			}

			return base.TryConvertVersionToInt(StringValue, out OutValue, Hint);
		}

		private static bool TryConvertVersionToIntInternal(string? StringValue, out ulong OutValue)
		{
			if (StringValue == null)
			{
				OutValue = 0;
				return false;
			}

			// helper function to convert from month name to month index
			int GetMonthIndex(string MonthName)
			{
				string[] MonthNames = { "Jan", "Feb", "Mar", "Apr", "May", "Jun", "Jul", "Aug", "Sep", "Oct", "Nov", "Dec" };
				int MonthIndex = MonthNames.FindIndex(X => MonthName.StartsWith(X, StringComparison.OrdinalIgnoreCase));
				return (MonthIndex >= 0) ? MonthIndex + 1 : 0;
			}

			// PGDK update (QFE) installer file name from GitHub. GDK-MonthName_YYYY_Update_QQ (note: earlier PGDK releases did not have the 'GDK-' prefix)
			Regex PGDKQFEPattern = new(@"^(?:GDK-)?([A-Za-z]+)_20(\d\d)_Update_(\d+)?$");
			Match Result = PGDKQFEPattern.Match(StringValue);
			if (Result.Success)
			{
				int MonthIndex = GetMonthIndex(Result.Groups[1].Value);
				if (MonthIndex > 0)
				{
					string PGDKVersion = Result.Groups[2].Value + MonthIndex.ToString("D2") + Int32.Parse(Result.Groups[3].Value).ToString("D2") + "00";
					return UInt64.TryParse(PGDKVersion, System.Globalization.NumberStyles.HexNumber, System.Globalization.CultureInfo.InvariantCulture, out OutValue);
				}
			}

			// PGDK base installer file name for from GitHub. GDK-MonthName_YYYY or GDK-MonthName_YYYY_SomeSuffix (note: earlier PGDK releases did not have the 'GDK-' prefix)
			Regex PGDKBasePattern = new(@"^(?:GDK-)?([A-Za-z]+)_20(\d\d)(?:_[A-Za-z]+)?$");
			Result = PGDKBasePattern.Match(StringValue);
			if (Result.Success)
			{
				int MonthIndex = GetMonthIndex(Result.Groups[1].Value);
				if (MonthIndex > 0)
				{
					string PGDKVersion = Result.Groups[2].Value + MonthIndex.ToString("D2") + "0000";
					return UInt64.TryParse(PGDKVersion, System.Globalization.NumberStyles.HexNumber, System.Globalization.CultureInfo.InvariantCulture, out OutValue);
				}
			}

			OutValue = 0;
			return false;
		}
	}
}
