// Copyright Epic Games, Inc. All Rights Reserved.

using EpicGames.Core;
using Microsoft.Extensions.Logging;
using System.Collections.Generic;

namespace UnrealBuildTool
{
	/// <summary>
	/// WinGDK project generator - a thin wrapper around WindowsProjectGenerator
	/// </summary>
	class WinGDKProjectGenerator : WindowsProjectGenerator
	{
		/// <summary>
		/// Constructor
		/// </summary>
		/// <param name="Arguments">Command line arguments passed to the project generator</param>
		/// <param name="Logger"></param>
#pragma warning disable IDE0060 // Remove unused parameter - constructor is found by reflection in GenerateProjectFilesMode.ExecuteAsync
		public WinGDKProjectGenerator(CommandLineArguments Arguments, ILogger Logger)
			: base(Arguments, Logger)
#pragma warning restore IDE0060
		{
		}

		/// <inheritdoc/>
		public override IEnumerable<UnrealTargetPlatform> GetPlatforms()
		{
			yield return UnrealTargetPlatform.WinGDK;
		}

		/// <inheritdoc/>
		public override string GetVisualStudioPlatformName(VSSettings InVSSettings)
		{
			return "x64";
		}

		/// <inheritdoc/>
		public override bool RequiresDistinctVisualStudioConfigurationName(VSSettings InVSSettings)
		{
			// WinGDK and Win64 cannot both use the "x64" platform name unless they have different configuration names.
			// this gives us "Development|x64" for Win64 and "WinGDK_Development|x64" for WinGDK
			// otherwise we would not be able to F5 debug both of them and their properties would overwrite each other.
			return true;
		}
	}
}
