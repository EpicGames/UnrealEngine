// Copyright Epic Games, Inc. All Rights Reserved.

using EpicGames.Core;
using Microsoft.Extensions.Logging;
using System;
using System.IO;
using System.Linq;

namespace UnrealBuildTool
{
	class WinGDKToolChain : MSGamingToolChain
	{
		protected override bool SupportsContentOnlyProjects => true;

		public WinGDKToolChain(ReadOnlyTargetRules Target, ILogger Logger)
			: base(Target, Logger)
		{
		}

		public override void FinalizeOutput(ReadOnlyTargetRules Target, TargetMakefileBuilder MakefileBuilder, DirectoryReference ExeDir)
		{
			base.FinalizeOutput(Target, MakefileBuilder, ExeDir);

			// give additional tips for debugging content-only projects
			bool bIsUnrealGame = (Target.Type != TargetType.Program) && (UEBuildTarget.GetAppNameForTargetType(Target.Type) == Target.Name);
			if (bIsUnrealGame && Environment.CommandLine.Contains("-frommsbuild", StringComparison.OrdinalIgnoreCase))
			{
				FileReference? ProjectFile = GDKContentOnlyProjectSupport.GetProjectFileFromTarget(Target, Logger, bUseLog: false); // don't log as that's just been done in base.FinalizeOutput
				if (ProjectFile != null)
				{
					DirectoryReference ProjectSourceDir = DirectoryReference.Combine(ProjectFile.Directory, "Source");
					bool bHasCode = (DirectoryReference.Exists(ProjectSourceDir) && DirectoryReference.EnumerateFiles(ProjectSourceDir, "*.target.cs", SearchOption.TopDirectoryOnly).Any());
					if (!bHasCode)
					{
						string BinaryDirectory = MakefileBuilder.Makefile.ExecutableFile.Directory.ToString();
						string BaseDir = Path.GetFullPath(Path.Combine(BinaryDirectory, "../../../"));
						string RelativeBinaryDir = Utils.MakePathRelativeTo(BinaryDirectory, BaseDir);
						string RelativeProjectDir = Utils.MakePathRelativeTo(ProjectFile.FullName, ProjectFile.Directory.ParentDirectory!.FullName);
						string StageDir = Path.Combine(ProjectFile.Directory.FullName, "Saved", "StagedBuilds", Target.Platform.ToString());
						string ProjectBaseDir = Path.Combine(StageDir, RelativeBinaryDir).Replace('\\', '/');
						if (!Directory.Exists(StageDir))
						{
							Logger.LogWarning("Project staging directory does not exist: {StageDir}. Check that the game is staged.", StageDir);
						}

						Logger.LogInformation("*** For content-only F5 debugging to work, make sure your VS debugger command line starts with: ***\n../../../{RelativeProjectDir} -BaseDir=\"{ProjectBaseDir}\"\n********", RelativeProjectDir, ProjectBaseDir);
					}
				}
			}
		}
	}
}
