// Copyright Epic Games, Inc. All Rights Reserved.

#include "WinGDKPlatformApplicationMisc.h"

void FWinGDKPlatformApplicationMisc::SetHighDPIMode()
{
	// do nothing on WinGDK - Microsoft says SetProcessDpiAwareness does not work well on containerized packages
}

