// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Windows/WindowsPlatformApplicationMisc.h"

struct APPLICATIONCORE_API FWinGDKPlatformApplicationMisc : public FWindowsPlatformApplicationMisc
{
	static void SetHighDPIMode();
};

typedef FWinGDKPlatformApplicationMisc FPlatformApplicationMisc;
