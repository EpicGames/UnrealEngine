// Copyright Epic Games, Inc. All Rights Reserved.

using EpicGames.Core;
using UnrealBuildTool;


public class Core_WinGDK : Core
{
	public Core_WinGDK(ReadOnlyTargetRules Target) : base(Target)
	{
		// Also included by target platform modules, so make sure we're on the target platform
		if (Target.Platform == UnrealTargetPlatform.WinGDK)
		{
			PrivateIncludePathModuleNames.Add("GDKRuntime");
			PrivateDependencyModuleNames.Add("GRDK");

			if (Target.Configuration != UnrealTargetConfiguration.Shipping)
			{
				PublicDefinitions.Add("UE_MEMORY_TRACE_AVAILABLE=1");
				PublicDefinitions.Add("PLATFORM_SUPPORTS_TRACE_WIN32_MODULE_DIAGNOSTICS=1");
				PublicDefinitions.Add("PLATFORM_SUPPORTS_TRACE_WIN32_CALLSTACK=1");
				PublicDefinitions.Add("PLATFORM_SUPPORTS_PLATFORM_EVENTS=1");
				PublicSystemLibraries.Add("advapi32.lib");
			}
		}
	}
}
