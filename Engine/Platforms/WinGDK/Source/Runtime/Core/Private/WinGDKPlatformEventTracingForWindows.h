// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Windows/AllowWindowsPlatformTypes.h"
#	include <evntrace.h>
#	include <evntcons.h>
#include "Windows/HideWindowsPlatformTypes.h"
