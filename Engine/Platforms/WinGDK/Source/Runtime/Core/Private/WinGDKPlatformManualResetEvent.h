// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Microsoft/MicrosoftPlatformManualResetEvent.h"

namespace UE { using FPlatformManualResetEvent = UE::HAL::Private::FMicrosoftManualResetEvent; }
