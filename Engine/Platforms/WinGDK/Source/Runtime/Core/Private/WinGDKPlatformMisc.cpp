// Copyright Epic Games, Inc. All Rights Reserved.

#include "WinGDKPlatformMisc.h"
#include "Misc/ConfigCacheIni.h"
#include "Misc/CoreDelegates.h"
#include "GDKRuntimeModule.h"

void FWinGDKPlatformMisc::PlatformPreInit()
{
	auto InitGDKRuntime = []()
	{
		FModuleManager::Get().LoadModuleChecked(TEXT("GDKRuntime"));
		UE_CLOG(!IGDKRuntimeModule::Get().IsAvailable(), LogInit, Fatal, TEXT("GDK runtime is not available") );
	};

#if WITH_GRDK_DEV_RUNTIME_INIT
	// dev init may require config to be ready first
	if (GConfig && GConfig->IsReadyForUse())
	{
		InitGDKRuntime();
	}
	else
	{
		FCoreDelegates::TSConfigReadyForUse().AddLambda(InitGDKRuntime);
	}
#else
	InitGDKRuntime();
#endif // WITH_GRDK_DEV_RUNTIME_INIT



	FWindowsPlatformMisc::PlatformPreInit();
}

void FWinGDKPlatformMisc::PlatformInit()
{
	FModuleManager::Get().LoadModule(TEXT("GDKVirtualKeyboard"));

	FWindowsPlatformMisc::PlatformInit();
}

ENetworkConnectionType FWinGDKPlatformMisc::GetNetworkConnectionType()
{
	return IGDKRuntimeModule::Get().GetNetworkConnectionType();
}

bool FWinGDKPlatformMisc::HasActiveWiFiConnection()
{
	return IGDKRuntimeModule::Get().HasActiveWiFiConnection();
}


