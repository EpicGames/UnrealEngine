// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Windows/WindowsPlatform.h"

#undef WINDOWS_USE_FEATURE_APPLICATIONMISC_CLASS
#define WINDOWS_USE_FEATURE_APPLICATIONMISC_CLASS			0

#undef WINDOWS_USE_FEATURE_PLATFORMMISC_CLASS
#define WINDOWS_USE_FEATURE_PLATFORMMISC_CLASS				0

#undef WINDOWS_USE_FEATURE_PLATFORMHTTP_CLASS
#define WINDOWS_USE_FEATURE_PLATFORMHTTP_CLASS				0

#undef PLATFORM_CAN_SUPPORT_EDITORONLY_DATA

