// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Microsoft/MSVCPlatformCompilerPreSetup.h"

#include <winapifamily.h>
