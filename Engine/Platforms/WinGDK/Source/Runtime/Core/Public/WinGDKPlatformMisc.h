// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Windows/WindowsPlatformMisc.h"

struct CORE_API FWinGDKPlatformMisc : public FWindowsPlatformMisc
{
	static void PlatformPreInit();
	static void PlatformInit();
	
	static ENetworkConnectionType GetNetworkConnectionType();
	static bool HasActiveWiFiConnection();
};

typedef FWinGDKPlatformMisc FPlatformMisc;
