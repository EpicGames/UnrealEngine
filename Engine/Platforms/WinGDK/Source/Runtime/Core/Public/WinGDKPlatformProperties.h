// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

// since we include another platform's header, we have to undefine the special define, so that we don't double typedef
#ifdef PROPERTY_HEADER_SHOULD_DEFINE_TYPE
#define SAVED_PROPERTY_HEADER_SHOULD_DEFINE_TYPE
#undef PROPERTY_HEADER_SHOULD_DEFINE_TYPE
#endif

#include "Windows/WindowsPlatformProperties.h"

template<bool IS_CLIENT_ONLY>
class FWinGDKPlatformProperties : public FWindowsPlatformProperties<false, false, IS_CLIENT_ONLY>
{
public:
	typedef FWindowsPlatformProperties<false, false, IS_CLIENT_ONLY> TBaseWindowsPlatformProperties;

	static constexpr inline const char* IniPlatformName()
	{
		return "WinGDK";
	}

	static constexpr inline const char* PlatformName()
	{
		if (IS_CLIENT_ONLY)
		{
			return "WinGDKClient";
		}

		return "WinGDK";
	}

	static constexpr inline bool RequiresCookedData()
	{
		return true;
	}

	static constexpr inline bool IsClientOnly()
	{
		return IS_CLIENT_ONLY;
	}

};

#ifdef SAVED_PROPERTY_HEADER_SHOULD_DEFINE_TYPE
typedef FWinGDKPlatformProperties<!WITH_SERVER_CODE && !WITH_EDITOR> FPlatformProperties;
#endif
