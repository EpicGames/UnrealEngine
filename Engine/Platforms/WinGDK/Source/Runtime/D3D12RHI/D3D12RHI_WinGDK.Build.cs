// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class D3D12RHI_WinGDK : D3D12RHI
{
	public D3D12RHI_WinGDK(ReadOnlyTargetRules Target) : base(Target)
	{
	}
}
