// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Windows/WindowsD3D12Submission.h"
