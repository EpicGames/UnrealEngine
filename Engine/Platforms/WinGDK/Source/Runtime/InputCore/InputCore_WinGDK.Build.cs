// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;
using EpicGames.Core;


public class InputCore_WinGDK : InputCore
{
	public InputCore_WinGDK(ReadOnlyTargetRules Target) : base(Target)
	{
	}
}
