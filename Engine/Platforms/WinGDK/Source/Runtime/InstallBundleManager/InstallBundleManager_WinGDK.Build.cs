// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class InstallBundleManager_WinGDK : InstallBundleManager
{
	public InstallBundleManager_WinGDK(ReadOnlyTargetRules Target) : base(Target)
	{
		PublicDefinitions.Add("WITH_PLATFORM_INSTALL_BUNDLE_SOURCE=1");
	}
}
