// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;
using EpicGames.Core;


public class BackgroundHTTP_WinGDK : BackgroundHTTP
{
	public BackgroundHTTP_WinGDK(ReadOnlyTargetRules Target) : base(Target)
	{
	}
}
