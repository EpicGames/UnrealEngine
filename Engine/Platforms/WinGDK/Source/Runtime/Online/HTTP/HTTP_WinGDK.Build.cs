// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class HTTP_WinGDK : HTTP
{
	public HTTP_WinGDK(ReadOnlyTargetRules Target) : base(Target)
	{
	}
}
