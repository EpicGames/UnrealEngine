// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;
using System;
using System.IO;

public class VulkanRHI_WinGDK : VulkanRHI
{
	public VulkanRHI_WinGDK(ReadOnlyTargetRules Target) : base(Target)
	{
	}
}
